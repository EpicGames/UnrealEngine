// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API KISMETCOMPILER_API

class UClass;

/** Utility functions for ClassFlags that is reused between normal and skeleton-only compilation. */
class FBlueprintClassFlagUtils
{
public:
	/** Given a class, let it inherit class flags from its parent class. Only considers inheritable flags. */
	static UE_API void PropagateParentClassFlags_CompileClassLayout(UClass* Class);

	/** Given a class, let it inherit class flags from its parent class. Only considers inheritable flags. */
	static UE_API void PropagateParentClassFlags_FinishCompilingClass(UClass* Class);

	/** Given a class, set any class flags that are based on its properties. Does not consider parent class properties. */
	static UE_API void AppendPropertyBasedClassFlags(UClass* Class);
};

#undef UE_API
