// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintClassFlagUtils.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"

void FBlueprintClassFlagUtils::PropagateParentClassFlags_CompileClassLayout(UClass* Class)
{
	UClass* ParentClass = Class->GetSuperClass();
	Class->ClassFlags |= (ParentClass->ClassFlags & CLASS_Inherit);
	Class->ClassCastFlags |= ParentClass->ClassCastFlags;
}

void FBlueprintClassFlagUtils::PropagateParentClassFlags_FinishCompilingClass(UClass* Class)
{
	UClass* ParentClass = Class->GetSuperClass();
	Class->ClassFlags &= ~CLASS_RecompilerClear;
	Class->ClassFlags |= (ParentClass->ClassFlags & CLASS_ScriptInherit);//@TODO: ChangeParentClass had this, but I don't think I want it: | UClass::StaticClassFlags;  // will end up with CLASS_Intrinsic
	Class->ClassCastFlags |= ParentClass->ClassCastFlags;
	Class->ClassConfigName = ParentClass->ClassConfigName;
	Class->ClassWithin = ParentClass->ClassWithin;
}

void FBlueprintClassFlagUtils::AppendPropertyBasedClassFlags(UClass* Class)
{
	for (TFieldIterator<FProperty> It(Class, EFieldIteratorFlags::ExcludeSuper); It; ++It)
	{
		FProperty* Property = *It;

		// If any property is instanced, then the class needs to also have CLASS_HasInstancedReference flag
		if (Property->ContainsInstancedObjectProperty())
		{
			Class->ClassFlags |= CLASS_HasInstancedReference;
		}

		// Look for OnRep 
		if (Property->HasAnyPropertyFlags(CPF_Net))
		{
			// Verify rep notifies are valid, if not, clear them
			if (Property->HasAnyPropertyFlags(CPF_RepNotify))
			{
				UFunction* OnRepFunc = Class->FindFunctionByName(Property->RepNotifyFunc);
				if (OnRepFunc != NULL && OnRepFunc->NumParms == 0 && OnRepFunc->GetReturnProperty() == NULL)
				{
					// This function is good so just continue
					continue;
				}
				// Invalid function for RepNotify! clear the flag
				Property->RepNotifyFunc = NAME_None;
			}
		}
		if (Property->HasAnyPropertyFlags(CPF_Config))
		{
			// If we have properties that are set from the config, then the class needs to also have CLASS_Config flags
			Class->ClassFlags |= CLASS_Config;
		}
	}
}