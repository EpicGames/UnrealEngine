// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	EdGraphCompiler.cpp
=============================================================================*/

#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphCompilerUtilities.h"
#include "EdGraphUtilities.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Templates/SubclassOf.h"
#include "UObject/ObjectPtr.h"
#include "Misc/CommandLine.h"

namespace UE::Private::GraphCompilerContext
{
	// Builds against old assets routinely generate deprecation messages, fixing them would defeat the point of running compat-testing builds.
	// Since we expect deprecated BP to continue to be functional, provide a way to make the warnings simple log messages.
	bool IgnoreBlueprintDeprecationWarnings()
	{
		const static bool bIgnoreBpWarnings = FParse::Param(FCommandLine::Get(), TEXT("IgnoreBlueprintDeprecationWarnings"));
		return bIgnoreBpWarnings;
	}
}

//////////////////////////////////////////////////////////////////////////
// FGraphCompilerContext

/** Validates that the interconnection between two pins is schema compatible */
void FGraphCompilerContext::ValidateLink(const UEdGraphPin* PinA, const UEdGraphPin* PinB) const
{
	const bool bPinDirectionsValid = ((PinA->Direction == EGPD_Input) && (PinB->Direction == EGPD_Output)) || ((PinA->Direction == EGPD_Output) && (PinB->Direction == EGPD_Input));
	if (!bPinDirectionsValid)
	{
		MessageLog.Error(TEXT("Direction mismatch between pins @@ and @@"), PinA, PinB);
	}

	if (PinA->GetOwningNodeUnchecked() == PinB->GetOwningNodeUnchecked())
	{
		MessageLog.Error(TEXT("Pins @@ and @@ on the same node @@ are connected to each other, creating a loop."), PinA, PinB, PinA->GetOwningNode());
	}
}

/** Validate that the wiring for a single pin is schema compatible */
void FGraphCompilerContext::ValidatePin(const UEdGraphPin* Pin) const
{
	if (!Pin || !Pin->GetOwningNodeUnchecked())
	{
		MessageLog.Error(
			*FText::Format(
				NSLOCTEXT("EdGraphCompiler", "PinWrongOuterErrorFmt", "Blueprint is corrupted! Pin '{0}' has wrong outer '{1}'."),
				FText::FromString(Pin ? *Pin->GetName() : TEXT("UNKNOWN")),
				FText::FromString((Pin && Pin->GetOuter()) ? *Pin->GetOuter()->GetName() : TEXT("NULL"))
			).ToString(),
			Pin
		);
		return;
	}

	const int32 ErrorNum = MessageLog.NumErrors;

	// Validate the links to each connected pin
	for (int32 LinkIndex = 0; (LinkIndex < Pin->LinkedTo.Num()) && (ErrorNum == MessageLog.NumErrors); ++LinkIndex)
	{
		if (const UEdGraphPin* OtherPin = Pin->LinkedTo[LinkIndex])
		{
			ValidateLink(Pin, OtherPin);
		}
		else
		{
			MessageLog.Error(*NSLOCTEXT("EdGraphCompiler", "PinLinkIsNull Error", "Null or missing pin linked to @@").ToString(), Pin);
		}
	}
}

/** Validates that the node is schema compatible */
void FGraphCompilerContext::ValidateNode(const UEdGraphNode* Node) const
{
	const bool bIsNodeDeprecated = Node->IsDeprecated();
	if (bIsNodeDeprecated || Node->HasDeprecatedReference())
	{
		EEdGraphNodeDeprecationType ResponseType = bIsNodeDeprecated ? EEdGraphNodeDeprecationType::NodeTypeIsDeprecated : EEdGraphNodeDeprecationType::NodeHasDeprecatedReference;
		FEdGraphNodeDeprecationResponse Response = Node->GetDeprecationResponse(ResponseType);
		if (Response.MessageType != EEdGraphNodeDeprecationMessageType::None && !Response.MessageText.IsEmpty())
		{
			switch (Response.MessageType)
			{
			case EEdGraphNodeDeprecationMessageType::Warning:
				if (LIKELY(!UE::Private::GraphCompilerContext::IgnoreBlueprintDeprecationWarnings()))
				{
					MessageLog.Warning(*Response.MessageText.ToString(), Node);
					break;
				}
				// else intentional fall-through to Note
			case EEdGraphNodeDeprecationMessageType::Note:
				MessageLog.Note(*Response.MessageText.ToString(), Node);
				break;
			}
		}
	}

	for (int32 PinIndex = 0; PinIndex < Node->Pins.Num(); ++PinIndex)
	{
		if (const UEdGraphPin* Pin = Node->Pins[PinIndex])
		{
			UEdGraphNode* OwningNode = Pin->GetOwningNodeUnchecked();
			if (OwningNode != Node)
			{
				MessageLog.Error(*NSLOCTEXT("EdGraphCompiler", "WrongPinsOwner_Error", "The pin @@ has outer @@, but it's used in @@").ToString(), Pin, OwningNode, Node);
			}
			else
			{
				ValidatePin(Pin);
			}
		}
	}

	Node->ValidateNodeDuringCompilation(MessageLog);
}


/** Performs standard validation on the graph (outputs point to inputs, no more than one connection to each input, types match on both ends, etc...) */
bool FGraphCompilerContext::ValidateGraphIsWellFormed(UEdGraph* Graph) const
{
	int32 SavedErrorCount = MessageLog.NumErrors;

	for (int32 NodeIndex = 0; NodeIndex < Graph->Nodes.Num(); ++NodeIndex)
	{
		const UEdGraphNode* Node = Graph->Nodes[NodeIndex];
		if( Node )
		{
			ValidateNode(Node);
		}
		else
		{
			// The graph has a gap in its Nodes array, probably due to a deprecated node class.  Remove the element.
			Graph->Nodes.RemoveAt(NodeIndex);
			NodeIndex--;
		}
	}

	return SavedErrorCount == MessageLog.NumErrors;
}

UEdGraphNode* FGraphCompilerContext::FindNodeByClass(const UEdGraph* Graph, TSubclassOf<UEdGraphNode>  NodeClass, bool bExpectedUnique) const
{
	UEdGraphNode* FirstResultNode = NULL;

	for (int32 NodeIndex = 0; NodeIndex < Graph->Nodes.Num(); ++NodeIndex)
	{
		UEdGraphNode* Node = Graph->Nodes[NodeIndex];
		if (Node->IsA(NodeClass))
		{
			if (bExpectedUnique)
			{
				if (FirstResultNode == NULL)
				{
					FirstResultNode = Node;
				}
				else
				{
					MessageLog.Error(*FString::Printf(TEXT("Expected only one %s node in graph @@, but found both @@ and @@"), *(NodeClass->GetName())), Graph, FirstResultNode, Node);
				}
			}
			else
			{
				return Node;
			}
		}
	}

	return FirstResultNode;
}

/** Prunes any nodes that weren't visited from the graph, printing out a warning */
void FGraphCompilerContext::PruneIsolatedNodes(const TArray<UEdGraphNode*>& RootSet, TArray<UEdGraphNode*>& GraphNodes)
{
	FEdGraphUtilities::FNodeVisitor Visitor;

	for (TArray<UEdGraphNode*>::TConstIterator It(RootSet); It; ++It)
	{
		UEdGraphNode* RootNode = *It;
		Visitor.TraverseNodes(RootNode);
	}

	for (int32 NodeIndex = 0; NodeIndex < GraphNodes.Num(); ++NodeIndex)
	{
		UEdGraphNode* Node = GraphNodes[NodeIndex];
		if (!Visitor.VisitedNodes.Contains(Node))
		{
			if (!CanIgnoreNode(Node))
			{
				// Disabled this warning, because having orphaned chains is standard workflow for LDs
				//MessageLog.Warning(TEXT("Node @@ will never be executed and is being pruned"), Node);
			}

			if (!ShouldForceKeepNode(Node))
			{
				Node->BreakAllNodeLinks();
				GraphNodes.RemoveAtSwap(NodeIndex);
				--NodeIndex;
			}
		}
	}
}

/**
 * Performs a topological sort on the graph of nodes passed in (which is expected to form a DAG), scheduling them.
 * If there are cycles or unconnected nodes present in the graph, an error will be output for each node that failed to be scheduled.
 */
void FGraphCompilerContext::CreateExecutionSchedule(const TArray<UEdGraphNode*>& GraphNodes, /*out*/ TArray<UEdGraphNode*>& LinearExecutionSchedule) const
{
	TArray<UEdGraphNode*> NodesWithNoEdges;
	TMap<UEdGraphNode*, int32> NumIncomingEdges;
	int32 TotalGraphEdgesLeft = 0;

	// Build a list of nodes with no antecedents and update the initial incoming edge counts for every node
	for (int32 NodeIndex = 0; NodeIndex < GraphNodes.Num(); ++NodeIndex)
	{
		UEdGraphNode* Node = GraphNodes[NodeIndex];

		const int32 NumEdges = CountIncomingEdges(Node);
		NumIncomingEdges.Add(Node, NumEdges);
		TotalGraphEdgesLeft += NumEdges;
			
		if (NumEdges == 0)
		{
			NodesWithNoEdges.Add(Node);
		}
	}

	// While there are nodes with no unscheduled inputs, schedule them and queue up any that are newly scheduleable
	while (NodesWithNoEdges.Num() > 0)
	{
		// Schedule a node
		UEdGraphNode* Node = NodesWithNoEdges[0];
		NodesWithNoEdges.RemoveAtSwap(0);
		LinearExecutionSchedule.Add(Node);

		// Decrement edge counts for things that depend on this node, and queue up any that hit 0 incoming edges
		for (int32 PinIndex = 0; PinIndex < Node->Pins.Num(); ++PinIndex)
		{
			UEdGraphPin* OutPin = Node->Pins[PinIndex];
			if ((OutPin->Direction == EGPD_Output) && PinIsImportantForDependancies(OutPin))
			{
				for (int32 LinkIndex = 0; LinkIndex < OutPin->LinkedTo.Num(); ++LinkIndex)
				{
					UEdGraphPin* LinkedToPin = OutPin->LinkedTo[LinkIndex];
					if( !LinkedToPin )
					{
						// If something went wrong in serialization and we have a bad connection, skip this.
						continue;
					}

					UEdGraphNode* WasDependentNode = OutPin->LinkedTo[LinkIndex]->GetOwningNodeUnchecked();
					int32* pNumEdgesLeft = WasDependentNode ? NumIncomingEdges.Find(WasDependentNode) : NULL;

					// Remove the edge between these two nodes, since node is scheduled
					if (pNumEdgesLeft)
					{
						int32& NumEdgesLeft = *pNumEdgesLeft;

						if (NumEdgesLeft <= 0)
						{
							MessageLog.Error(TEXT("Internal compiler error inside CreateExecutionSchedule (site 1); there is an issue with node/pin manipulation that was performed in this graph, please contact the Blueprints team!"));
							LinearExecutionSchedule.Empty();
							return;
						}
						NumEdgesLeft--;
						TotalGraphEdgesLeft--;

						// Was I the last link on that node?
						if (NumEdgesLeft == 0)
						{
							NodesWithNoEdges.Add(WasDependentNode);
						}
					}
					else
					{
						MessageLog.Error(TEXT("Internal compiler error inside CreateExecutionSchedule (site 2); there is an issue with node/pin manipulation that was performed in this graph, please contact the Blueprints team!"));
						LinearExecutionSchedule.Empty();
						return;
					}
				}
			}
		}
	}

	// At this point, any remaining edges are either within an unrelated subgraph (unconnected island) or indicate loops that break the DAG constraint
	// Before this is called, any unconnected graphs should have been cut free so we can just error
	if (TotalGraphEdgesLeft > 0)
	{
		// Run thru and print out any nodes that couldn't be scheduled due to loops
		for (TMap<UEdGraphNode*, int32>::TIterator It(NumIncomingEdges); It; ++It)
		{
			//@TODO: Probably want to determine the actual pin that caused the cycle, instead of just printing out the node
			if (It.Value() > 0)
			{
				MessageLog.Error(TEXT("Dependency cycle detected, preventing node @@ from being scheduled"), It.Key());
			}
		}
	}
}
