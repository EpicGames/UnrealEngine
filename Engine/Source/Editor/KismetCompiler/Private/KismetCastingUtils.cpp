// Copyright Epic Games, Inc. All Rights Reserved.

#include "KismetCastingUtils.h"

#include "BPTerminal.h"
#include "BlueprintCompiledStatement.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Kismet/BlueprintTypeConversions.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompilerMisc.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Optional.h"
#include "Templates/Casts.h"
#include "Templates/Tuple.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

namespace UE::KismetCompiler::CastingUtils
{

FBPTerminal* MakeImplicitCastTerminal(FKismetFunctionContext& Context, UEdGraphPin* Net, UEdGraphNode* SourceNode)
{
	check(Net);
	
	FBPTerminal* Result = Context.CreateLocalTerminal();
	check(Result);

	Result->CopyFromPin(Net, Context.NetNameMap->MakeValidName(Net, TEXT("ImplicitCast")));
	Result->Source = (SourceNode ? SourceNode : Net->GetOwningNode());

	return Result;
}

void RegisterImplicitCasts(FKismetFunctionContext& Context)
{
	auto AddCastMapping = [&Context](UEdGraphPin* DestinationPin, const FConversion& Conversion)
	{
		check(DestinationPin);

		FBPTerminal* NewTerm = MakeImplicitCastTerminal(Context, DestinationPin);
		UEdGraphNode* OwningNode = DestinationPin->GetOwningNode();

		Context.ImplicitCastMap.Add(DestinationPin, FImplicitCastParams{Conversion, NewTerm, OwningNode});
	};

	// The current context's NetMap can be a mix of input and output pin types.
	// We need to check both pin types in order to get adequate coverage for potential cast points.
	for (const auto& It : Context.NetMap)
	{
		UEdGraphPin* CurrentPin = It.Key;
		check(CurrentPin);

		bool bIsConnectedOutput =
			(CurrentPin->Direction == EGPD_Output) && (CurrentPin->LinkedTo.Num() > 0);

		bool bIsConnectedInput =
			(CurrentPin->Direction == EGPD_Input) && (CurrentPin->LinkedTo.Num() > 0);

		if (bIsConnectedOutput)
		{
			for (UEdGraphPin* DestinationPin : CurrentPin->LinkedTo)
			{
				check(DestinationPin);

				if (Context.ImplicitCastMap.Contains(DestinationPin))
				{
					continue;
				}

				FConversion Conversion = GetFloatingPointConversion(*CurrentPin, *DestinationPin);
				if (Conversion.Type != FloatingPointCastType::None)
				{
					AddCastMapping(DestinationPin, Conversion);
				}
			}
		}
		else if (bIsConnectedInput)
		{
			if (Context.ImplicitCastMap.Contains(CurrentPin))
			{
				continue;
			}

			if (CurrentPin->LinkedTo.Num() > 0)
			{
				const UEdGraphPin* SourcePin = CurrentPin->LinkedTo[0];
				check(SourcePin);

				FConversion Conversion = GetFloatingPointConversion(*SourcePin, *CurrentPin);
				if (Conversion.Type != FloatingPointCastType::None)
				{
					AddCastMapping(CurrentPin, Conversion);
				}
			}
		}
	}
}

void InsertImplicitCastStatement(FKismetFunctionContext& Context, const FImplicitCastParams& CastParams, FBPTerminal* RHSTerm)
{
	check(RHSTerm);
	check(CastParams.TargetTerminal);
	check(CastParams.TargetNode);

	EKismetCompiledStatementType CompiledStatementType = {};
	UFunction* FunctionToCall = nullptr;

	switch (CastParams.Conversion.Type)
	{
	case FloatingPointCastType::FloatToDouble:
		CompiledStatementType = KCST_FloatToDoubleCast;
		break;

	case FloatingPointCastType::DoubleToFloat:
		CompiledStatementType = KCST_DoubleToFloatCast;
		break;

	case FloatingPointCastType::Container:
	case FloatingPointCastType::Struct:
		CompiledStatementType = KCST_CallFunction;
		FunctionToCall = CastParams.Conversion.Function;
		check(FunctionToCall);
		break;

	default:
		check(false);
		break;
	}

	FBlueprintCompiledStatement& CastStatement = Context.AppendStatementForNode(CastParams.TargetNode);
	CastStatement.Type = CompiledStatementType;
	CastStatement.FunctionToCall = FunctionToCall;
	CastStatement.LHS = CastParams.TargetTerminal;
	CastStatement.RHS.Add(RHSTerm);
}

FBPTerminal* InsertImplicitCastStatement(FKismetFunctionContext& Context, UEdGraphPin* DestinationPin, FBPTerminal* RHSTerm)
{
	check(DestinationPin);
	check(RHSTerm);

	FBPTerminal* Result = nullptr;

	const FImplicitCastParams* CastParams =
		Context.ImplicitCastMap.Find(DestinationPin);

	if (CastParams != nullptr)
	{
		InsertImplicitCastStatement(Context, *CastParams, RHSTerm);

		Result = CastParams->TargetTerminal;

		// Removal of the pin entry indicates to the compiler that the implicit cast has been processed.
		Context.ImplicitCastMap.Remove(DestinationPin);
	}

	return Result;
}

bool RemoveRegisteredImplicitCast(FKismetFunctionContext& Context, const UEdGraphPin* DestinationPin)
{
	check(DestinationPin);

	int32 RemovedCount = Context.ImplicitCastMap.Remove(DestinationPin);

	return (RemovedCount > 0);
}

FConversion GetFloatingPointConversion(const UEdGraphPin& SourcePin, const UEdGraphPin& DestinationPin)
{
	using namespace UE::Kismet::BlueprintTypeConversions;

	UClass* BlueprintTypeConversionsClass = UBlueprintTypeConversions::StaticClass();
	
	UFunction* ArrayConversionFunction = BlueprintTypeConversionsClass->FindFunctionByName(TEXT("ConvertArrayType"));
	check(ArrayConversionFunction);

	UFunction* SetConversionFunction = BlueprintTypeConversionsClass->FindFunctionByName(TEXT("ConvertSetType"));
	check(SetConversionFunction);

	UFunction* MapConversionFunction = BlueprintTypeConversionsClass->FindFunctionByName(TEXT("ConvertMapType"));
	check(MapConversionFunction);

	if (SourcePin.PinType.IsMap() && DestinationPin.PinType.IsMap())
	{
		if ((SourcePin.PinType.PinValueType.TerminalCategory == UEdGraphSchema_K2::PC_Real) && (DestinationPin.PinType.PinValueType.TerminalCategory == UEdGraphSchema_K2::PC_Real))
		{
			if ((SourcePin.PinType.PinValueType.TerminalSubCategory == UEdGraphSchema_K2::PC_Float) && ((DestinationPin.PinType.PinValueType.TerminalSubCategory == UEdGraphSchema_K2::PC_Double)))
			{
				return {FloatingPointCastType::Container, MapConversionFunction};
			}
			else if ((SourcePin.PinType.PinValueType.TerminalSubCategory == UEdGraphSchema_K2::PC_Double) && ((DestinationPin.PinType.PinValueType.TerminalSubCategory == UEdGraphSchema_K2::PC_Float)))
			{
				return {FloatingPointCastType::Container, MapConversionFunction};
			}
		}
		else if ((SourcePin.PinType.PinValueType.TerminalCategory == UEdGraphSchema_K2::PC_Struct) && (DestinationPin.PinType.PinValueType.TerminalCategory == UEdGraphSchema_K2::PC_Struct))
		{
			UScriptStruct* SourceStruct = Cast<UScriptStruct>(SourcePin.PinType.PinValueType.TerminalSubCategoryObject.Get());
			UScriptStruct* DestinationStruct = Cast<UScriptStruct>(DestinationPin.PinType.PinValueType.TerminalSubCategoryObject.Get());

			if (FStructConversionTable::Get().GetConversionFunction(SourceStruct, DestinationStruct).IsSet())
			{
				return {FloatingPointCastType::Container, MapConversionFunction};
			}
		}
	}

	if ((SourcePin.PinType.PinCategory == UEdGraphSchema_K2::PC_Real) && (DestinationPin.PinType.PinCategory == UEdGraphSchema_K2::PC_Real))
	{
		if ((SourcePin.PinType.PinSubCategory == UEdGraphSchema_K2::PC_Float) && ((DestinationPin.PinType.PinSubCategory == UEdGraphSchema_K2::PC_Double)))
		{
			if (SourcePin.PinType.IsArray())
			{
				return {FloatingPointCastType::Container, ArrayConversionFunction};
			}
			else if (SourcePin.PinType.IsSet())
			{
				return {FloatingPointCastType::Container, SetConversionFunction};
			}
			else if (SourcePin.PinType.IsMap())
			{
				return {FloatingPointCastType::Container, MapConversionFunction};
			}
			else
			{
				return {FloatingPointCastType::FloatToDouble, nullptr};
			}
		}
		else if ((SourcePin.PinType.PinSubCategory == UEdGraphSchema_K2::PC_Double) && ((DestinationPin.PinType.PinSubCategory == UEdGraphSchema_K2::PC_Float)))
		{
			if (SourcePin.PinType.IsArray())
			{
				return {FloatingPointCastType::Container, ArrayConversionFunction};
			}
			else if (SourcePin.PinType.IsSet())
			{
				return {FloatingPointCastType::Container, SetConversionFunction};
			}
			else if (SourcePin.PinType.IsMap())
			{
				return {FloatingPointCastType::Container, MapConversionFunction};
			}
			else
			{
				return {FloatingPointCastType::DoubleToFloat, nullptr};
			}
		}
	}
	else if ((SourcePin.PinType.PinCategory == UEdGraphSchema_K2::PC_Struct) && (DestinationPin.PinType.PinCategory == UEdGraphSchema_K2::PC_Struct))
	{
		UScriptStruct* SourceStruct = Cast<UScriptStruct>(SourcePin.PinType.PinSubCategoryObject.Get());
		UScriptStruct* DestinationStruct = Cast<UScriptStruct>(DestinationPin.PinType.PinSubCategoryObject.Get());

		// Invalid BPs can have missing PinSubCategoryObject values if there was trouble loading the source struct
		if (SourceStruct && DestinationStruct)
		{
			if (TOptional<ConversionFunctionPairT> ConversionPair = FStructConversionTable::Get().GetConversionFunction(SourceStruct, DestinationStruct))
			{
				if (SourcePin.PinType.IsArray())
				{
					return { FloatingPointCastType::Container, ArrayConversionFunction };
				}
				else if (SourcePin.PinType.IsSet())
				{
					return { FloatingPointCastType::Container, SetConversionFunction };
				}
				else if (SourcePin.PinType.IsMap())
				{
					return { FloatingPointCastType::Container, MapConversionFunction };
				}
				else
				{
					return { FloatingPointCastType::Struct, ConversionPair->Get<1>() };
				}
			}
		}
		else
		{
			UE_CLOGF(!SourceStruct, LogK2Compiler, Warning, "Source pin '%ls' had null struct object (%ls)", *SourcePin.GetName(), *SourcePin.GetOwningNode()->GetFullName());
			UE_CLOGF(!DestinationStruct, LogK2Compiler, Warning, "Destination pin '%ls' had null struct object (%ls)", *DestinationPin.GetName(), *DestinationPin.GetOwningNode()->GetFullName());
		}
	}

	return {FloatingPointCastType::None, nullptr};
}

} // namespace UE::KismetCompiler::CastingUtils
