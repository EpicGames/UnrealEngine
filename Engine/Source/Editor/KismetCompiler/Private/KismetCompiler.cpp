// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	KismetCompiler.cpp
=============================================================================*/


#include "KismetCompiler.h"
#include "BlueprintClassFlagUtils.h"
#include "Algo/Find.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Misc/CoreMisc.h"
#include "Components/ActorComponent.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "UObject/UObjectHash.h"
#include "UObject/MetaData.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "Serialization/ArchiveObjectCrc32.h"
#include "GameFramework/Actor.h"
#include "EdGraphNode_Comment.h"
#include "Curves/CurveBase.h"
#include "Engine/Engine.h"
#include "Editor/EditorEngine.h"
#include "Components/TimelineComponent.h"
#include "Engine/TimelineTemplate.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Blueprint/BlueprintExtension.h"
#include "EdGraphUtilities.h"
#include "INotifyFieldValueChanged.h"
#include "K2Node_AddDelegate.h"
#include "K2Node_CallFunction.h"
#include "K2Node_Composite.h"
#include "K2Node_CreateDelegate.h"
#include "K2Node_CustomEvent.h"
#include "K2Node_Event.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_FunctionResult.h"
#include "K2Node_GeneratedBoundEvent.h"
#include "K2Node_Knot.h"
#include "K2Node_MacroInstance.h"
#include "K2Node_MakeArray.h"
#include "K2Node_TemporaryVariable.h"
#include "K2Node_Timeline.h"
#include "K2Node_Tunnel.h"
#include "K2Node_TunnelBoundary.h"
#include "K2Node_VariableGet.h"
#include "K2Node_VariableSet.h"
#include "K2Node_EditablePinBase.h" // for FUserPinInfo
#include "KismetCastingUtils.h"
#include "KismetCompilerBackend.h"
#include "Kismet2/KismetReinstanceUtilities.h"
#include "Engine/SCS_Node.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "ScriptDisassembler.h"
#include "ComponentTypeRegistry.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "UserDefinedStructureCompilerUtils.h"
#include "K2Node_EnumLiteral.h"
#include "K2Node_SetVariableOnPersistentFrame.h"
#include "EdGraph/EdGraphNode_Documentation.h"
#include "Engine/DynamicBlueprintBinding.h"
#include "Engine/InheritableComponentHandler.h"
#include "Serialization/ArchiveScriptReferenceCollector.h"
#include "UObject/UnrealTypePrivate.h"
#include "KismetCompilerModule.h"

namespace UE::Private
{
	KISMETCOMPILER_API bool (*GCanSaveFunctionsForPackage)(const UPackage*) = nullptr;
}

static bool bDebugPropertyPropagation = false;

FSimpleMulticastDelegate FKismetCompilerContext::OnPreCompile;
FSimpleMulticastDelegate FKismetCompilerContext::OnPostCompile;
FKismetCompilerContext::FOnPreCompileBlueprint FKismetCompilerContext::OnPreCompileBlueprint;

#define USE_TRANSIENT_SKELETON 0

#define LOCTEXT_NAMESPACE "KismetCompiler"

//////////////////////////////////////////////////////////////////////////
// Stats for this module
DECLARE_CYCLE_STAT(TEXT("Create Schema"), EKismetCompilerStats_CreateSchema, STATGROUP_KismetCompiler );
DECLARE_CYCLE_STAT(TEXT("Create Function List"), EKismetCompilerStats_CreateFunctionList, STATGROUP_KismetCompiler );
DECLARE_CYCLE_STAT(TEXT("Expansion"), EKismetCompilerStats_Expansion, STATGROUP_KismetCompiler)
DECLARE_CYCLE_STAT(TEXT("Validate"), EKismetCompilerStats_Validate, STATGROUP_KismetCompiler )
DECLARE_CYCLE_STAT(TEXT("Expand Node"), EKismetCompilerStats_ExpandNode, STATGROUP_KismetCompiler )
DECLARE_CYCLE_STAT(TEXT("Post Expansion Step"), EKismetCompilerStats_PostExpansionStep, STATGROUP_KismetCompiler )
DECLARE_CYCLE_STAT(TEXT("Process uber"), EKismetCompilerStats_ProcessUbergraph, STATGROUP_KismetCompiler );
DECLARE_CYCLE_STAT(TEXT("Process func"), EKismetCompilerStats_ProcessFunctionGraph, STATGROUP_KismetCompiler );
DECLARE_CYCLE_STAT(TEXT("Generate Function Graph"), EKismetCompilerStats_GenerateFunctionGraphs, STATGROUP_KismetCompiler );
DECLARE_CYCLE_STAT(TEXT("Precompile Function"), EKismetCompilerStats_PrecompileFunction, STATGROUP_KismetCompiler );
DECLARE_CYCLE_STAT(TEXT("Compile Function"), EKismetCompilerStats_CompileFunction, STATGROUP_KismetCompiler);
DECLARE_CYCLE_STAT(TEXT("Create Locals and Register Nets"), EKismetCompilerStats_CreateLocalsAndRegisterNets, STATGROUP_KismetCompiler );
DECLARE_CYCLE_STAT(TEXT("Postcompile Function"), EKismetCompilerStats_PostcompileFunction, STATGROUP_KismetCompiler );
DECLARE_CYCLE_STAT(TEXT("Finalization"), EKismetCompilerStats_FinalizationWork, STATGROUP_KismetCompiler );
DECLARE_CYCLE_STAT(TEXT("Code Gen"), EKismetCompilerStats_CodeGenerationTime, STATGROUP_KismetCompiler);
DECLARE_CYCLE_STAT(TEXT("Clean and Sanitize Class"), EKismetCompilerStats_CleanAndSanitizeClass, STATGROUP_KismetCompiler );
DECLARE_CYCLE_STAT(TEXT("Create Class Properties"), EKismetCompilerStats_CreateClassVariables, STATGROUP_KismetCompiler );
DECLARE_CYCLE_STAT(TEXT("Bind and Link Class"), EKismetCompilerStats_BindAndLinkClass, STATGROUP_KismetCompiler );
DECLARE_CYCLE_STAT(TEXT("Calculate checksum of CDO"), EKismetCompilerStats_ChecksumCDO, STATGROUP_KismetCompiler );
DECLARE_CYCLE_STAT(TEXT("Calculate checksum of signature"), EKismetCompilerStats_ChecksumSignature, STATGROUP_KismetCompiler);
DECLARE_CYCLE_STAT(TEXT("Pruning"), EKismetCompilerStats_PruneIsolatedNodes, STATGROUP_KismetCompiler);
DECLARE_CYCLE_STAT(TEXT("Merge Ubergraph Pages In"), EKismetCompilerStats_MergeUbergraphPagesIn, STATGROUP_KismetCompiler);

namespace UE::KismetCompiler::Private
{
	// The function collects all nodes, that can represents entry points of the execution. Any node connected to "root" node (by execution link) won't be consider isolated.
	static void GatherRootSet(const UEdGraph* Graph, TArray<UEdGraphNode*>& RootSet, bool bIncludeNodesThatCouldBeExpandedToRootSet)
	{
		for (UEdGraphNode* Node : Graph->Nodes)
		{
			const bool bRootSetByType = Node && (Node->IsA<UK2Node_FunctionEntry>() || Node->IsA<UK2Node_Event>() || Node->IsA<UK2Node_Timeline>());
			UK2Node* K2Node = Cast<UK2Node>(Node);
			bool bIsRootSet = bRootSetByType || (K2Node && K2Node->IsNodeRootSet());

			if (Node && bIncludeNodesThatCouldBeExpandedToRootSet && !bIsRootSet)
			{
				//Include non-pure K2Nodes, without input pins
				auto HasInputPins = [](UK2Node* InNode) -> bool
				{
					for (UEdGraphPin* Pin : InNode->Pins)
					{
						if (Pin && (EEdGraphPinDirection::EGPD_Input == Pin->Direction))
						{
							return true;
						}
					}
					return false;
				};

				bIsRootSet |= K2Node && !K2Node->IsNodePure() && !HasInputPins(K2Node);
			}
			if (bIsRootSet)
			{
				RootSet.Add(Node);
			}
		}
	}

	// When we change pins back to real/float types, we also expect that the series of pin types matches that of the UFunction.
	// If there's ever a discrepancy, then there's not much we can do other than log a warning.
	// It typically means that the BP is in a bad state, which prevents us from deducing the corresponding pin.
	static void RestoreFloatPinsToNode(UK2Node_EditablePinBase* Node, const UFunction* FunctionSignature, bool bProcessOutputParamsOnly = false)
	{
		check(Node);
		check(FunctionSignature);

		int UserDefinedPinCursor = 0;
		int PinCursor = 0;

		if (Node->UserDefinedPins.Num() > 0)
		{
			bool bMatchingPinFound = false;
			
			check(Node->UserDefinedPins[0]);

			FName UserPinName = Node->UserDefinedPins[0]->PinName;

			for (int i = 0; i < Node->Pins.Num(); ++i)
			{
				check(Node->Pins[i]);

				// Subpins aren't a part of the function signature, so we need to skip them.
				if (Node->Pins[i]->ParentPin)
				{
					continue;
				}

				if (Node->Pins[i]->PinName == UserPinName)
				{
					PinCursor = i;
					bMatchingPinFound = true;
					break;
				}
			}

			if (!bMatchingPinFound)
			{
				UE_LOGF(LogK2Compiler, Warning, "User pin '%ls' ('%ls') was not found in the pins list!",
					*UserPinName.ToString(),
					*Node->GetFullName());
			}
		}

		auto IsFloatProperty = [Node](const FProperty* Property)
		{
			check(Property);

			if (const FFloatProperty* FloatProperty = CastField<FFloatProperty>(Property))
			{
				return true;
			}
			else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
			{
				check(ArrayProperty->Inner);
				return ArrayProperty->Inner->IsA<FFloatProperty>();
			}
			else if (const FSetProperty* SetProperty = CastField<FSetProperty>(Property))
			{
				check(SetProperty->ElementProp);
				return SetProperty->ElementProp->IsA<FFloatProperty>();
			}
			else if (const FMapProperty* MapProperty = CastField<FMapProperty>(Property))
			{
				check(MapProperty->KeyProp);
				check(MapProperty->ValueProp);

				if (MapProperty->KeyProp->IsA<FFloatProperty>() || MapProperty->ValueProp->IsA<FFloatProperty>())
				{
					UE_LOGF(LogK2Compiler, Warning, "A map with float entries was found in '%ls'. Delegate signature may be inaccurate.",
						*Node->GetFullName());
				}

				return false;
			}

			return false;
		};

		auto IsValidParameter = [bProcessOutputParamsOnly](FProperty* Property)
		{
			check(Property);

			bool bIsOutputParameter =
				(Property->HasAnyPropertyFlags(CPF_OutParm) && !Property->HasAnyPropertyFlags(CPF_ConstParm));

			if (bProcessOutputParamsOnly)
			{
				return bIsOutputParameter;
			}
			else
			{
				// HACK: We still have to reckon with arrays that are implicitly copy-by-ref.
				// Ideally, we should be excluding all output params.
				bool bIsInputParameter =
					(Property->IsA<FArrayProperty>() && Property->HasAnyPropertyFlags(CPF_Parm)) ||
					!bIsOutputParameter;

				return bIsInputParameter;
			}
		};

		for (TFieldIterator<FProperty> PropIt(FunctionSignature); PropIt; ++PropIt)
		{
			FProperty* CurrentProperty = *PropIt;
			check(CurrentProperty);

			if (IsValidParameter(CurrentProperty))
			{
				if (UserDefinedPinCursor < Node->UserDefinedPins.Num())
				{
					if (IsFloatProperty(CurrentProperty))
					{
						TSharedPtr<FUserPinInfo>& UserPin = Node->UserDefinedPins[UserDefinedPinCursor];
						check(UserPin);

						if (UserPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Real)
						{
							UserPin->PinType.PinSubCategory = UEdGraphSchema_K2::PC_Float;
						}
						else
						{
							UE_LOGF(LogK2Compiler, Warning, "Expected a 'real' type for user pin '%ls' ('%ls'), but found type '%ls' instead!",
								*UserPin->PinName.ToString(),
								*Node->GetFullName(),
								*UserPin->PinType.PinCategory.ToString());
						}

						// Subpins aren't a part of the function signature, so we need to skip them.
						while (PinCursor < Node->Pins.Num())
						{
							UEdGraphPin* Pin = Node->Pins[PinCursor];
							check(Pin);
							if (Pin->ParentPin == nullptr)
							{
								break;
							}

							++PinCursor;
						}

						if (PinCursor < Node->Pins.Num())
						{
							UEdGraphPin* Pin = Node->Pins[PinCursor];
							check(Pin);

							if (Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Real)
							{
								Pin->PinType.PinSubCategory = UEdGraphSchema_K2::PC_Float;
							}
							else
							{
								UE_LOGF(LogK2Compiler, Warning, "Expected a 'real' type for pin '%ls' ('%ls'), but found type '%ls' instead!",
									*Pin->PinName.ToString(),
									*Node->GetFullName(),
									*Pin->PinType.PinCategory.ToString());
							}
						}
						else
						{
							UE_LOGF(LogK2Compiler, Warning, "PinCursor ('%ls') has an invalid index: %d!",
								*Node->GetFullName(),
								PinCursor);
						}
					}

					++UserDefinedPinCursor;
					++PinCursor;
				}
				else
				{
					// It's possible that we'll use up pins before reaching the end of the function signature's properties.
					// This can happen when a function has reference params. The corresponding pins are split between two nodes (ie: entry and result nodes).
					break;
				}
			}
		}
	}

	static UFunction* GetLinkedDelegateSignature(UK2Node* NodeWithDelegate)
	{
		check(NodeWithDelegate);

		const UEdGraphPin* DelegateOutPin = NodeWithDelegate->FindPin(UK2Node_Event::DelegateOutputName);
		if (DelegateOutPin && (DelegateOutPin->LinkedTo.Num() > 0))
		{
			// Since the BP hasn't been compiled yet, we'll run into knot nodes that need to be skipped.
			check(DelegateOutPin->LinkedTo[0]);
			UEdGraphNode* LinkedNode = DelegateOutPin->LinkedTo[0]->GetOwningNode();
			while (UK2Node_Knot* KnotNode = Cast<UK2Node_Knot>(LinkedNode))
			{
				check(KnotNode->GetOutputPin());
				UEdGraphPin* OutputPin = KnotNode->GetOutputPin();
				check(OutputPin);

				// Knots don't necessarily have to be linked.
				// In these cases, there simply isn't a corresponding signature to reference.
				LinkedNode = (OutputPin->LinkedTo.Num() > 0) ? OutputPin->LinkedTo[0]->GetOwningNode() : nullptr;
			}

			// If the delegate pin has been linked, then we should run into either a
			// AddDelegate or RemoveDelegate node.
			// In either case, it should have the signature that we need.
			UK2Node_BaseMCDelegate* DelegateNode = Cast<UK2Node_BaseMCDelegate>(LinkedNode);
			if (DelegateNode)
			{
				return DelegateNode->GetDelegateSignature();
			}
		}

		return nullptr;
	}

	// By default, float pins are changed to real/double pins during serialization.
	// However, Blueprint functions that are bound to native delegates need to use real/float pins if
	// that's what the underlying delegate signature requires.
	// Failure to do so will lead to incorrect UFunction signatures in the skeleton class,
	// which can cause compilation errors.
	static void MatchNodesToDelegateSignatures(const UBlueprint* BP)
	{
		check(BP);

		TArray<UK2Node_CreateDelegate*> CreateDelegateNodes;
		FBlueprintEditorUtils::GetAllNodesOfClass(BP, CreateDelegateNodes);

		TMap<FName, UK2Node_CreateDelegate*> CreateDelegateNodeMap;
		for (UK2Node_CreateDelegate* CreateDelegateNode : CreateDelegateNodes)
		{
			check(CreateDelegateNode);
			if (CreateDelegateNode->SelectedFunctionName != NAME_None)
			{
				CreateDelegateNodeMap.Add(CreateDelegateNode->SelectedFunctionName, CreateDelegateNode);
			}
		}

		// Handle function terminator nodes

		TArray<UK2Node_FunctionTerminator*> TerminatorNodes;
		FBlueprintEditorUtils::GetAllNodesOfClass(BP, TerminatorNodes);
		for (UK2Node_FunctionTerminator* FunctionTerminatorNode : TerminatorNodes)
		{
			check(FunctionTerminatorNode);

			// If the function is an override, then we can't change the signature.
			UFunction* Function = FunctionTerminatorNode->FunctionReference.ResolveMember<UFunction>(BP->ParentClass);
			if (!Function)
			{
				UK2Node_CreateDelegate** DelegateNodePtr = CreateDelegateNodeMap.Find(FunctionTerminatorNode->FunctionReference.GetMemberName());
				if (DelegateNodePtr)
				{
					UK2Node_CreateDelegate* DelegateNode = *DelegateNodePtr;
					check(DelegateNode);

					UFunction* DelegateSignature = DelegateNode->GetDelegateSignature();

					if (DelegateSignature == nullptr)
					{
						// In some edge cases, the delegate pin for UK2Node_CreateDelegate might be missing PinSubCategoryMemberReference information.
						// Our fallback case is to find the signature from the linked K2Node_AddDelegate. 
						DelegateSignature = GetLinkedDelegateSignature(DelegateNode);
					}

					if (DelegateSignature)
					{
						if (DelegateSignature->GetNativeFunc())
						{
							bool bUseOutputParams =
								FunctionTerminatorNode->IsA<UK2Node_FunctionResult>();

							RestoreFloatPinsToNode(FunctionTerminatorNode, DelegateSignature, bUseOutputParams);
						}
					}
					else
					{
						UE_LOGF(LogK2Compiler, Warning, "Unable to determine delegate signature for node '%ls'!",
							*DelegateNode->GetFullName());
					}
				}
			}
		}

		// Handle custom event nodes

		TArray<UK2Node_CustomEvent*> CustomEventNodes;
		FBlueprintEditorUtils::GetAllNodesOfClass(BP, CustomEventNodes);
		for (UK2Node_CustomEvent* CustomEventNode : CustomEventNodes)
		{
			check(CustomEventNode);

			// We'll give precedence to the signature of the linked delegate.
			// Failing that, we'll then see if the custom event was possibly created via a CreateDelegate node.
			UFunction* DelegateSignature = GetLinkedDelegateSignature(CustomEventNode);

			if (DelegateSignature && DelegateSignature->GetNativeFunc())
			{
				RestoreFloatPinsToNode(CustomEventNode, DelegateSignature);
			}
			else
			{
				FName NodeName = CustomEventNode->CustomFunctionName;

				UK2Node_CreateDelegate** DelegateNodePtr = CreateDelegateNodeMap.Find(NodeName);
				if (DelegateNodePtr)
				{
					UK2Node_CreateDelegate* DelegateNode = *DelegateNodePtr;
					check(DelegateNode);

					DelegateSignature = DelegateNode->GetDelegateSignature();

					if (DelegateSignature == nullptr)
					{
						// In some edge cases, the delegate pin for UK2Node_CreateDelegate might be missing PinSubCategoryMemberReference information.
						// Our fallback case is to find the signature from the linked K2Node_AddDelegate. 
						DelegateSignature = GetLinkedDelegateSignature(DelegateNode);
					}

					if (DelegateSignature)
					{
						if (DelegateSignature->GetNativeFunc())
						{
							RestoreFloatPinsToNode(CustomEventNode, DelegateSignature);
						}
					}
					else
					{
						UE_LOGF(LogK2Compiler, Warning, "Unable to determine delegate signature for node '%ls'!",
							*DelegateNode->GetFullName());
					}
				}
			}
		}
	}

	bool FindMacroCycle(const UK2Node_MacroInstance* RootNode, TSet<FGuid>& VisitedMacroGraphs, TArray<const UK2Node_MacroInstance*>& CurrentPath)
	{
		check(RootNode);
		if (UEdGraph* MacroGraph = RootNode->GetMacroGraph())
		{
			VisitedMacroGraphs.Add(MacroGraph->GraphGuid);
			CurrentPath.Push(RootNode);
			for (const UEdGraphNode* ChildNode : MacroGraph->Nodes)
			{
				if (const UK2Node_MacroInstance* MacroInstanceNode = Cast<const UK2Node_MacroInstance>(ChildNode))
				{
					UEdGraph* InnerMacroGraph = MacroInstanceNode->GetMacroGraph();
					if (VisitedMacroGraphs.Contains(InnerMacroGraph->GraphGuid))
					{
						return true;
					}
					else
					{
						return FindMacroCycle(MacroInstanceNode, VisitedMacroGraphs, CurrentPath);
					}
				}
			}
			CurrentPath.Pop();
		}

		return false;
	}

	bool FindAndReportMacroCycles(const UEdGraph* SourceGraph, FCompilerResultsLog& MessageLog)
	{
		check(SourceGraph);

		TArray<const UK2Node_MacroInstance*> RootLevelMacroNodes;
		for (const UEdGraphNode* Node : SourceGraph->Nodes)
		{
			if (const UK2Node_MacroInstance* MacroInstanceNode = Cast<const UK2Node_MacroInstance>(Node))
			{
				RootLevelMacroNodes.Add(MacroInstanceNode);
			}
		}

		for (const UK2Node_MacroInstance* RootNode : RootLevelMacroNodes)
		{
			TSet<FGuid> VisitedMacroGraphs;
			TArray<const UK2Node_MacroInstance*> CurrentPath;
			if (FindMacroCycle(RootNode, VisitedMacroGraphs, CurrentPath))
			{
				MessageLog.Error(*LOCTEXT("MacroCycleDetected_ErrorFmt", "Macro cycle detected in @@! Cycle path:").ToString(), RootNode);

				for (const UK2Node_MacroInstance* Node : CurrentPath)
				{
					MessageLog.Error(*LOCTEXT("MacroCyclePath_ErrorFmt", "\t@@").ToString(), Node);
				}

				return true;
			}
		}

		return false;
	}
}

//////////////////////////////////////////////////////////////////////////
// FKismetCompilerContext

FKismetCompilerContext::FKismetCompilerContext(UBlueprint* SourceSketch, FCompilerResultsLog& InMessageLog, const FKismetCompilerOptions& InCompilerOptions)
	: FGraphCompilerContext(InMessageLog)
	, Schema(NULL)
	, CompileOptions(InCompilerOptions)
	, Blueprint(SourceSketch)
	, NewClass(NULL)
	, OldClass(nullptr)
	, ConsolidatedEventGraph(NULL)
	, UbergraphContext(NULL)
	, bIsFullCompile(false)
	, OldCDO(nullptr)
	, OldGenLinkerIdx(INDEX_NONE)
	, OldLinker(nullptr)
	, TargetClass(nullptr)
	, bAssignDelegateSignatureFunction(false)
	, bAutomaticallyRegisterInputEvents(false)
{
	MacroRowMaxHeight = 0;

	MinimumSpawnX = -2000;
	MaximumSpawnX = 2000;

	AverageNodeWidth = 200;
	AverageNodeHeight = 150;

	HorizontalSectionPadding = 250;
	VerticalSectionPadding = 250;
	HorizontalNodePadding = 40;

	MacroSpawnX = MinimumSpawnX;
	MacroSpawnY = -2000;
	
	VectorStruct = TBaseStructure<FVector>::Get();
	RotatorStruct = TBaseStructure<FRotator>::Get();
	TransformStruct = TBaseStructure<FTransform>::Get();
	LinearColorStruct = TBaseStructure<FLinearColor>::Get();
}

FKismetCompilerContext::~FKismetCompilerContext()
{
	for (TMap< TSubclassOf<UEdGraphNode>, FNodeHandlingFunctor*>::TIterator It(NodeHandlers); It; ++It)
	{
		FNodeHandlingFunctor* FPtr = It.Value();
		delete FPtr;
	}
	NodeHandlers.Empty();
	DefaultPropertyValueMap.Empty();
}

UEdGraphSchema_K2* FKismetCompilerContext::CreateSchema()
{
	return NewObject<UEdGraphSchema_K2>();
}

void FKismetCompilerContext::EnsureProperGeneratedClass(UClass*& TargetUClass)
{
	if( TargetUClass && !((UObject*)TargetUClass)->IsA(UBlueprintGeneratedClass::StaticClass()) )
	{
		FKismetCompilerUtilities::ConsignToOblivion(TargetUClass, Blueprint->bIsRegeneratingOnLoad);
		TargetUClass = NULL;
	}
}

void FKismetCompilerContext::SpawnNewClass(const FString& NewClassName)
{
	// First, attempt to find the class, in case it hasn't been serialized in yet
	NewClass = FindObject<UBlueprintGeneratedClass>(Blueprint->GetOutermost(), *NewClassName);
	if (NewClass == NULL)
	{
		// If the class hasn't been found, then spawn a new one
		NewClass = NewObject<UBlueprintGeneratedClass>(Blueprint->GetOutermost(), FName(*NewClassName), RF_Public | RF_Transactional);
	}
	else
	{
		// Already existed, but wasn't linked in the Blueprint yet due to load ordering issues
		NewClass->ClassGeneratedBy = Blueprint;
		FBlueprintCompileReinstancer::Create(NewClass);
	}
}

void FKismetCompilerContext::FSubobjectCollection::AddObject(const UObject* const InObject)
{
	if ( InObject )
	{
		Collection.Add(InObject);
		ForEachObjectWithOuter(InObject, [this](UObject* Child)
		{
			Collection.Add(Child);
		});
	}
}

bool FKismetCompilerContext::FSubobjectCollection::operator()(const UObject* const RemovalCandidate) const
{
	return ( NULL != Collection.Find(RemovalCandidate) );
}

void FKismetCompilerContext::CleanAndSanitizeClass(UBlueprintGeneratedClass* ClassToClean, UObject*& InOldCDO)
{
	BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_CleanAndSanitizeClass);

	const bool bRecompilingOnLoad = Blueprint->bIsRegeneratingOnLoad;
	FString TransientClassString = FString::Printf(TEXT("TRASHCLASS_%s"), *Blueprint->GetName());
	FName TransientClassName = MakeUniqueObjectName(GetTransientPackage(), UBlueprintGeneratedClass::StaticClass(), FName(*TransientClassString));
	UClass* TransientClass = NewObject<UBlueprintGeneratedClass>(GetTransientPackage(), TransientClassName, RF_Public | RF_Transient);
	
	UClass* ParentClass = Blueprint->ParentClass;

	if(CompileOptions.CompileType == EKismetCompileType::SkeletonOnly)
	{
		if(UBlueprint* BlueprintParent = Cast<UBlueprint>(Blueprint->ParentClass->ClassGeneratedBy))
		{
			ParentClass = BlueprintParent->SkeletonGeneratedClass;
		}
	}

	if( ParentClass == NULL )
	{
		ParentClass = UObject::StaticClass();
	}
	TransientClass->CppClassStaticFunctions = ParentClass->CppClassStaticFunctions;
	TransientClass->ClassGeneratedBy = Blueprint;
	TransientClass->ClassFlags |= CLASS_CompiledFromBlueprint|CLASS_NewerVersionExists;

	SetNewClass( ClassToClean );
	InOldCDO = ClassToClean->GetDefaultObject(false); // we don't need to create the CDO at this point
	
	const ERenameFlags RenFlags = REN_DontCreateRedirectors | REN_NonTransactional | REN_DoNotDirty;

	if( InOldCDO )
	{
		FString TransientCDOString = FString::Printf(TEXT("TRASH_%s"), *InOldCDO->GetName());
		FName TransientCDOName = MakeUniqueObjectName(GetTransientPackage(), TransientClass, FName(*TransientCDOString));

        // Rename will remove the renamed object's linker when moving to a new package so invalidate the export beforehand
		FLinkerLoad::InvalidateExport(InOldCDO);
		InOldCDO->Rename(*TransientCDOName.ToString(), GetTransientPackage(), RenFlags);
	}

	// Purge all subobjects (properties, functions, params) of the class, as they will be regenerated
	TArray<UObject*> ClassSubObjects;
	GetObjectsWithOuter(ClassToClean, ClassSubObjects, EGetObjectsFlags::None);

	{
		// Save subobjects, that won't be regenerated.
		FSubobjectCollection SubObjectsToSave;
		SaveSubObjectsFromCleanAndSanitizeClass(SubObjectsToSave, ClassToClean);

		ClassSubObjects.RemoveAllSwap(SubObjectsToSave);
	}

	UClass* InheritableComponentHandlerClass = UInheritableComponentHandler::StaticClass();

	for( UObject* CurrSubObj : ClassSubObjects )
	{
		// ICH and ICH templates do not need to be destroyed in this way.. doing so will invalidate
		// transaction buffer references to these UObjects. The UBlueprint may not have a reference to
		// the ICH at the moment, and therefore might not have added it to SubObjectsToSave (and
		// removed the ICH from ClassSubObjects):
		if(Cast<UInheritableComponentHandler>(CurrSubObj) || CurrSubObj->IsInA(InheritableComponentHandlerClass) || CurrSubObj->HasAnyFlags(RF_InheritableComponentTemplate))
		{
			continue;
		}

		// Class properties are freed independently of GC, but functions we consign to the trash container object will persist until
		// the next GC pass, so we must purge serializable data first so we don't leak objects or crash due to invalidated references.
		if(UFunction* Function = Cast<UFunction>(CurrSubObj))
		{
			// Compiled function script (bytecode) may contain raw pointers to properties owned by this (or another) BP class. These
			// fields will be immediately freed after the compilation phase (see UClass::DestroyPropertiesPendingDestruction()), thus
			// invalidating any references to them in the "old" function object's serialized bytecode. Furthermore, reinstancing won't
			// update this function's bytecode, as that operation is only applied to a Blueprint class's dependencies, and does not
			// include "trash" class objects that we're creating here (see FBlueprintCompileReinstancer::UpdateBytecodeReferences()).
			// As we typically run a GC pass after BP compilation, this normally isn't an issue, because the "trash" class object that
			// owns this function object will get cleaned up at that point, preventing the "old" function object from being serialized
			// (e.g. as part of reinstancing an external dependency), and ensuring that we don't encounter one of these "dangling"
			// FField pointers. However, in certain cases (e.g. batched compiles) we may not run a GC pass in-between each operation,
			// so to cover that case, we ensure that existing bytecode is fully purged before moving a function to the "trash" class.
			Function->Script.Empty();

			// This array will get repopulated as part of constructing the new function object when compiling the class; we don't
			// want to preserve the old copy, because then the old function object could potentially be identified as a referencer
			// of a stale struct or a class asset during reference replacement if the previous dependency is subsequently recompiled.
			Function->ScriptAndPropertyObjectReferences.Empty();

			// We also need to destroy all child properties, as some may contain references to existing objects that can later be
			// invalidated as a result of compilation or GC, and also because we have cleared the references array above that's used
			// in ARO for reachability analysis during a GC pass. That means any references to objects owned by this class (e.g.
			// delegate signatures) are no longer seen as referenced by the function nor the class (due to the PurgeClass() below).
			// Additionally, references to any recompiled class objects or regenerated properties and functions won't be replaced
			// during the reinstancing phase, since "trash" class objects are meant for GC and will not be considered as a dependency.
			Function->DestroyChildPropertiesAndResetPropertyLinks();

			// Re-link to ensure that we also reset any cached data that's based on the (now empty) property list.
			Function->StaticLink(/*bRelinkExistingProperties =*/ true);
		}

		FName NewSubobjectName = MakeUniqueObjectName(TransientClass, CurrSubObj->GetClass(), CurrSubObj->GetFName());

        // Rename will remove the renamed object's linker when moving to a new package so invalidate the export beforehand
		FLinkerLoad::InvalidateExport(CurrSubObj);
		CurrSubObj->Rename(*NewSubobjectName.ToString(), TransientClass, RenFlags);
	}

	// Purge the class to get it back to a "base" state
	bool bLayoutChanging = ClassToClean->bLayoutChanging;
	ClassToClean->PurgeClass(bRecompilingOnLoad);

	// Set properties we need to regenerate the class with
	ClassToClean->PropertyLink = ParentClass->PropertyLink;
	ClassToClean->SetSuperStruct(ParentClass);
	ClassToClean->ClassWithin = ParentClass->ClassWithin ? ToRawPtr(ParentClass->ClassWithin) : ToRawPtr(UObject::StaticClass());
	ClassToClean->ClassConfigName = ClassToClean->IsNative() ? FName(ClassToClean->StaticConfigName()) : ParentClass->ClassConfigName;
	ClassToClean->DebugData = FBlueprintDebugData();

	if (bLayoutChanging)
	{
		ClassToClean->bLayoutChanging = true;
	}
}

void FKismetCompilerContext::SaveSubObjectsFromCleanAndSanitizeClass(FSubobjectCollection& SubObjectsToSave, UBlueprintGeneratedClass* ClassToClean)
{
	SubObjectsToSave.AddObjects(Blueprint->ComponentTemplates);
	SubObjectsToSave.AddObjects(Blueprint->Timelines);

	if ( Blueprint->SimpleConstructionScript )
	{
		SubObjectsToSave.AddObject(Blueprint->SimpleConstructionScript);
		if ( const USCS_Node* DefaultScene = Blueprint->SimpleConstructionScript->GetDefaultSceneRootNode() )
		{
			SubObjectsToSave.AddObject(DefaultScene->ComponentTemplate);
		}

		for ( USCS_Node* SCSNode : Blueprint->SimpleConstructionScript->GetAllNodes() )
		{
			SubObjectsToSave.AddObject(SCSNode->ComponentTemplate);
		}
	}

	{
		TSet<class UCurveBase*> Curves;
		for ( UTimelineTemplate* Timeline : Blueprint->Timelines )
		{
			if ( Timeline )
			{
				Timeline->GetAllCurves(Curves);
			}
		}
		for ( UActorComponent* Component : Blueprint->ComponentTemplates )
		{
			if ( UTimelineComponent* TimelineComponent = Cast<UTimelineComponent>(Component) )
			{
				TimelineComponent->GetAllCurves(Curves);
			}
		}
		for ( UCurveBase* Curve : Curves )
		{
			SubObjectsToSave.AddObject(Curve);
		}
	}

	if (Blueprint->InheritableComponentHandler)
	{
		SubObjectsToSave.AddObject(Blueprint->InheritableComponentHandler);
		TArray<UActorComponent*> AllTemplates;
		const bool bIncludeTransientTemplates = true;
		Blueprint->InheritableComponentHandler->GetAllTemplates(AllTemplates, bIncludeTransientTemplates);
		SubObjectsToSave.AddObjects(AllTemplates);
	}
}

void FKismetCompilerContext::PostCreateSchema()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PostCreateSchema);
	NodeHandlers.Add(UEdGraphNode_Comment::StaticClass(), new FNodeHandlingFunctor(*this));

	TArray<UClass*> ClassesOfUK2Node;
	GetDerivedClasses(UK2Node::StaticClass(), ClassesOfUK2Node, true);
	for (UClass* Class : ClassesOfUK2Node)
	{
		if( !Class->HasAnyClassFlags(CLASS_Abstract) )
		{
			UObject* CDO = Class->GetDefaultObject();
			UK2Node* K2CDO = Class->GetDefaultObject<UK2Node>();
			FNodeHandlingFunctor* HandlingFunctor = K2CDO->CreateNodeHandler(*this);
			if (HandlingFunctor)
			{
				NodeHandlers.Add(Class, HandlingFunctor);
			}
		}
	}
}	


/** Validates that the interconnection between two pins is schema compatible */
void FKismetCompilerContext::ValidateLink(const UEdGraphPin* PinA, const UEdGraphPin* PinB) const
{
	Super::ValidateLink(PinA, PinB);

	// We don't want to validate orphaned pin connections to avoid noisy connection errors that are
	// already being reported
	const bool bShouldValidatePinA = (PinA == nullptr || !PinA->bOrphanedPin);
	const bool bShouldValidatePinB = (PinB == nullptr || !PinB->bOrphanedPin);

	if (bShouldValidatePinA && bShouldValidatePinB)
	{
		// At this point we can assume the pins are linked, and as such the connection response should not be to disallow
		// @todo: Potentially revisit this later.
		// This API is intended to describe how to handle a potentially new connection to a pin that may already have a connection.
		// However it also checks all necessary constraints for a valid connection to exist. We rely on the fact that the "disallow"
		// response will be returned if the pins are not compatible; any other response here then means that the connection is valid.
		const FPinConnectionResponse ConnectResponse = Schema->CanCreateConnection(PinA, PinB);

		const bool bForbiddenConnection = (ConnectResponse.Response == CONNECT_RESPONSE_DISALLOW);
		const bool bMissingConversion = (ConnectResponse.Response == CONNECT_RESPONSE_MAKE_WITH_CONVERSION_NODE);
		if (bForbiddenConnection || bMissingConversion)
		{
			const FString ErrorMessage = FText::Format(LOCTEXT("PinTypeMismatch_ErrorFmt", "Can't connect pins @@ and @@: {0}"), ConnectResponse.Message).ToString();
			MessageLog.Error(*ErrorMessage, PinA, PinB);
		}
	}

	if (PinA && PinB && PinA->Direction != PinB->Direction)
	{
		const UEdGraphPin* InputPin = (EEdGraphPinDirection::EGPD_Input == PinA->Direction) ? PinA : PinB;
		const UEdGraphPin* OutputPin = (EEdGraphPinDirection::EGPD_Output == PinA->Direction) ? PinA : PinB;
		const bool bInvalidConnection = InputPin && OutputPin && (OutputPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Interface) && (InputPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object);
		if (bInvalidConnection)
		{
			MessageLog.Error(*LOCTEXT("PinTypeMismatch_Error_UseExplictCast", "Can't connect pins @@ (Interface) and @@ (Object). Use an explicit cast node.").ToString(), OutputPin, InputPin);
		}
	}
}

/** Validate that the wiring for a single pin is schema compatible */
void FKismetCompilerContext::ValidatePin(const UEdGraphPin* Pin) const
{
	Super::ValidatePin(Pin);

	UEdGraphNode* OwningNodeUnchecked = Pin ? Pin->GetOwningNodeUnchecked() : nullptr;
	if (!OwningNodeUnchecked)
	{
		//handled by Super::ValidatePin
		return;
	}

	if (Pin->LinkedTo.Num() > 1)
	{
		if (Pin->Direction == EGPD_Output)
		{
			if (Schema->IsExecPin(*Pin))
			{
				// Multiple outputs are not OK, since they don't have a clear defined order of execution
				MessageLog.Error(*LOCTEXT("TooManyOutputPinConnections_Error", "Exec output pin @@ cannot have more than one connection").ToString(), Pin);
			}
		}
		else if (Pin->Direction == EGPD_Input)
		{
			if (Schema->IsExecPin(*Pin))
			{
				// Multiple inputs to an execution wire are ok, it means we get executed from more than one path
			}
			else if( Schema->IsSelfPin(*Pin) )
			{
				// Pure functions and latent functions cannot have more than one self connection
				UK2Node_CallFunction* OwningNode = Cast<UK2Node_CallFunction>(OwningNodeUnchecked);
				if( OwningNode )
				{
					if( OwningNode->IsNodePure() )
					{
						MessageLog.Error(*LOCTEXT("PureFunction_OneSelfPin_Error", "Pure function call node @@ cannot have more than one self pin connection").ToString(), OwningNode);
					}
					else if( OwningNode->IsLatentFunction() )
					{
						MessageLog.Error(*LOCTEXT("LatentFunction_OneSelfPin_Error", "Latent function call node @@ cannot have more than one self pin connection").ToString(), OwningNode);
					}
				}
			}
			else
			{
				MessageLog.Error(*LOCTEXT("InputPin_OneConnection_Error", "Input pin @@ cannot have more than one connection").ToString(), Pin);
			}
		}
		else
		{
			MessageLog.Error(*LOCTEXT("UnexpectedPiNDirection_Error", "Unexpected pin direction encountered on @@").ToString(), Pin);
		}
	}

	//function return node exec pin should be connected to something
	if(Pin->Direction == EGPD_Input && Pin->LinkedTo.Num() == 0 && Schema->IsExecPin(*Pin) )
	{
		if (UK2Node_FunctionResult* OwningNode = Cast<UK2Node_FunctionResult>(OwningNodeUnchecked))
		{
			if(OwningNode->Pins.Num() > 1)
			{
				MessageLog.Warning(*LOCTEXT("ReturnNodeExecPinUnconnected", "ReturnNode Exec pin has no connections on @@").ToString(), Pin);
			}
		}
	}
}

/** Validates that the node is schema compatible */
void FKismetCompilerContext::ValidateNode(const UEdGraphNode* Node) const
{
	//@TODO: Validate the node type is a known one
	Super::ValidateNode(Node);
}

/** Creates a class variable */
FProperty* FKismetCompilerContext::CreateVariable(const FName VarName, const FEdGraphPinType& VarType)
{
	if (BPTYPE_FunctionLibrary == Blueprint->BlueprintType)
	{
		MessageLog.Error(
			*FText::Format(
				LOCTEXT("VariableInFunctionLibrary_ErrorFmt", "The variable {0} cannot be declared in FunctionLibrary @@"),
				FText::FromName(VarName)
			).ToString(),
			Blueprint
		);
	}

	FProperty* NewProperty = FKismetCompilerUtilities::CreatePropertyOnScope(NewClass, VarName, VarType, NewClass, CPF_None, Schema, MessageLog);
	if (NewProperty != nullptr)
	{
		FKismetCompilerUtilities::LinkAddedProperty(NewClass, NewProperty);
	}
	else
	{
		MessageLog.Error(
			*FText::Format(
				LOCTEXT("VariableInvalidType_ErrorFmt", "The variable {0} declared in @@ has an invalid type {1}"),
				FText::FromName(VarName),
				UEdGraphSchema_K2::TypeToText(VarType)
			).ToString(),
			Blueprint
		);
	}

	return NewProperty;
}

FMulticastDelegateProperty* FKismetCompilerContext::CreateMulticastDelegateVariable(const FName Name, const FEdGraphPinType& Type)
{
	// Note: We don't transact like OnAddNewDelegate since this method is intented to used during compilation only
	FMulticastDelegateProperty* NewMulticastDelegateProperty = CastField<FMulticastDelegateProperty>(CreateVariable(Name, Type));
	GeneratedMulticastDelegateProps.Add(NewMulticastDelegateProperty);
	return NewMulticastDelegateProperty;
}

FMulticastDelegateProperty* FKismetCompilerContext::CreateMulticastDelegateVariable(const FName Name)
{
	FEdGraphPinType MulticastDelegatePinType(UEdGraphSchema_K2::PC_MCDelegate, NAME_None, Blueprint->GeneratedClass, EPinContainerType::None, false, FEdGraphTerminalType());
	return CreateMulticastDelegateVariable(Name, MulticastDelegatePinType);
}

/** Determines if a node is pure */
bool FKismetCompilerContext::IsNodePure(const UEdGraphNode* Node) const
{
	if (const UK2Node* K2Node = Cast<const UK2Node>(Node))
	{
		return K2Node->IsNodePure();
	}
	// Only non K2Nodes are comments and documentation nodes, which are pure
	ensure(Node->IsA(UEdGraphNode_Comment::StaticClass())||Node->IsA(UEdGraphNode_Documentation::StaticClass()));
	return true;
}

void FKismetCompilerContext::ValidateVariableNames()
{
	UClass* ParentClass = Blueprint->ParentClass;
	if (ParentClass != nullptr)
	{
		TSharedPtr<FKismetNameValidator> ParentBPNameValidator;
		if (UBlueprint* ParentBP = Cast<UBlueprint>(Blueprint->ParentClass->ClassGeneratedBy))
		{
			ParentBPNameValidator = MakeShareable(new FKismetNameValidator(ParentBP));
		}

		TArray<FName> RemoveVars;
		for (FBPVariableDescription& VarDesc : Blueprint->NewVariables)
		{
			FName OldVarName = VarDesc.VarName;
			FName NewVarName = OldVarName;
			
			if (VarDesc.VarType.PinCategory == UEdGraphSchema_K2::PC_Struct)
			{
				if(!IsValid(VarDesc.VarType.PinSubCategoryObject.Get()))
				{
					VarDesc.VarType.PinSubCategoryObject = GetFallbackStruct();
				}
			}

			FString VarNameStr = OldVarName.ToString();
			if (ParentBPNameValidator.IsValid() && (ParentBPNameValidator->IsValid(VarNameStr) != EValidatorResult::Ok))
			{
				NewVarName = FBlueprintEditorUtils::FindUniqueKismetName(Blueprint, VarNameStr);
			}
			else if (ParentClass->IsNative()) // the above case handles when the parent is a blueprint
			{
				FFieldVariant ExistingField = FindUFieldOrFProperty(ParentClass, *VarNameStr);
				if (ExistingField)
				{
					UE_LOGF(LogK2Compiler, Warning, "ValidateVariableNames name %ls (used in %ls) is already taken by %ls"
						, *VarNameStr, *Blueprint->GetPathName(), *ExistingField.GetPathName());

					// If type is equivalent, we'll remove the blueprint variable and use the native variable wherever it's referenced
					if (Blueprint->GeneratedClass)
					{
						FFieldVariant ExistingBPClassField = FindUFieldOrFProperty(Blueprint->GeneratedClass, *VarNameStr);
						bool bIsCompatible = false;
						if (ExistingBPClassField)
						{
							if (const UObject* AsObject = ExistingField.ToUObject())
							{
								bIsCompatible = ExistingBPClassField.IsA(AsObject->GetClass());
							}
							else if (const FField* AsField = ExistingField.ToField())
							{
								bIsCompatible = ExistingBPClassField.IsA(AsField->GetClass());
							}
						}

						if (bIsCompatible)
						{
							RemoveVars.Add(OldVarName);
							continue;
						}
					}

					NewVarName = FBlueprintEditorUtils::FindUniqueKismetName(Blueprint, VarNameStr);
				}
			}

			if (OldVarName != NewVarName)
			{
				MessageLog.Warning(
					*FText::Format(
						LOCTEXT("MemberVariableConflictWarningFmt", "Found a member variable with a conflicting name ({0}) - changed to {1}."),
						FText::FromString(VarNameStr),
						FText::FromName(NewVarName)
					).ToString()
				);
				TGuardValue<bool> LockDependencies(Blueprint->bCachedDependenciesUpToDate, Blueprint->bCachedDependenciesUpToDate);
				FBlueprintEditorUtils::RenameMemberVariable(Blueprint, OldVarName, NewVarName);
			}
		}

		for (FName VarName : RemoveVars)
		{
			MessageLog.Note(
				*FText::Format(
					LOCTEXT("MemberVariableNativeConflictNoteFmt", "Found a member variable with a conflicting name ({0}) - replaced references with native parent class variable."),
					FText::FromName(VarName)
				).ToString()
			);

			int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, VarName);
			if (VarIndex != INDEX_NONE)
			{
				Blueprint->NewVariables.RemoveAt(VarIndex);
			}
		}
	}
}

void FKismetCompilerContext::ValidateComponentClassOverrides()
{
	if (UClass* ParentClass = Blueprint->ParentClass)
	{
		if (UObject* CDO = ParentClass->GetDefaultObject(false))
		{
			for (auto It(Blueprint->ComponentClassOverrides.CreateIterator()); It; ++It)
			{
				const FBPComponentClassOverride& Override = *It;
				if (UObject* OverridenObject = (UObject*)FindObjectWithOuter(CDO, nullptr, Override.ComponentName))
				{
					if (Override.ComponentClass && !Override.ComponentClass->IsChildOf(OverridenObject->GetClass()))
					{
						MessageLog.Error(
							*FText::Format(
								LOCTEXT("InvalidOverride", "{0} is not a legal override for component {1} because it does not derive from {2}."),
								FText::FromName(Override.ComponentClass->GetFName()),
								FText::FromName(Override.ComponentName),
								FText::FromName(OverridenObject->GetClass()->GetFName())
								).ToString()
						);
					}
				}
				else
				{
					MessageLog.Note(
						*FText::Format(
							LOCTEXT("UnneededOverride", "Removing class override for component {0} that no longer exists."),
							FText::FromName(Override.ComponentName)
						).ToString()
					);
					It.RemoveCurrent();
				}
			}
		}
	}
}

void FKismetCompilerContext::ValidateClassPropertyDefaults()
{
	// We cannot check the default value if there is no generated class yet (i.e. opening a BP for the first time)
	if (!Blueprint->GeneratedClass)
	{
		return;
	}

	UObject* CDO = Blueprint->GeneratedClass->GetDefaultObject(false);
	if (!CDO)
	{
		return;
	}

	// Check the New Variables in case the user has changed a type since the last check and a newer
	// CDO has not been generated yet
	for (FBPVariableDescription& VarDesc : Blueprint->NewVariables)
	{
		if (VarDesc.VarType.PinCategory == UEdGraphSchema_K2::PC_Class)
		{
			const UObject* ExpectedObject = VarDesc.VarType.PinSubCategoryObject.Get();
			const UClass* ExpectedClass = Cast<UClass>(ExpectedObject);

			if (FClassProperty* AsClassProperty = FindFProperty<FClassProperty>(Blueprint->GeneratedClass, VarDesc.VarName))
			{
				if (UObject* SetDefaultProp = AsClassProperty->GetObjectPropertyValue_InContainer(CDO))
				{
					// Object may be of a type that is also being compiled and therefore REINST_, so get the real class
					const UClass* SetDefaultClass = Cast<UClass>(SetDefaultProp);
					const UClass* RealClass = SetDefaultClass ? SetDefaultClass->GetAuthoritativeClass() : nullptr;

					if (RealClass && ExpectedClass && !RealClass->IsChildOf(ExpectedClass))
					{
						MessageLog.Error(
							*FText::Format(
								LOCTEXT("InvalidClassPropertyDefaultValue_ErrorFmt", "Invalid default type '{0}' on property '{1}' in @@"),
								FText::FromString(RealClass->GetName()), FText::FromString(AsClassProperty->GetName())
							).ToString(),
							Blueprint
						);
					}
				}
			}
		}
	}
}

void FKismetCompilerContext::ResetAndPopulateBlueprintGeneratedVariables()
{
	Blueprint->GeneratedVariables.Reset();
	PopulateBlueprintGeneratedVariables();
}

void FKismetCompilerContext::ValidateTimelineNames()
{
	TSharedPtr<FKismetNameValidator> ParentBPNameValidator;
	if( Blueprint->ParentClass != NULL )
	{
		UBlueprint* ParentBP = Cast<UBlueprint>(Blueprint->ParentClass->ClassGeneratedBy);
		if( ParentBP != NULL )
		{
			ParentBPNameValidator = MakeShareable(new FKismetNameValidator(ParentBP));
		}
	}

	for (int32 TimelineIndex=0; TimelineIndex < Blueprint->Timelines.Num(); ++TimelineIndex)
	{
		UTimelineTemplate* TimelineTemplate = Blueprint->Timelines[TimelineIndex];
		if( TimelineTemplate )
		{
			if( ParentBPNameValidator.IsValid() && ParentBPNameValidator->IsValid(TimelineTemplate->GetName()) != EValidatorResult::Ok )
			{
				// Use the viewer displayed Timeline name (without the _Template suffix) because it will be added later for appropriate checks.
				FName TimelineName = TimelineTemplate->GetVariableName();

				FName NewName = FBlueprintEditorUtils::FindUniqueKismetName(Blueprint, TimelineName.ToString());
				MessageLog.Warning(
					*FText::Format(
						LOCTEXT("TimelineConflictWarningFmt", "Found a timeline with a conflicting name ({0}) - changed to {1}."),
						FText::FromString(TimelineTemplate->GetName()),
						FText::FromName(NewName)
					).ToString()
				);
				FBlueprintEditorUtils::RenameTimeline(Blueprint, TimelineName, NewName);
			}
		}
	}
}

void FKismetCompilerContext::CreateClassVariablesFromBlueprint()
{
	BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_CreateClassVariables);

	check(NewClass);
	check(RepNotifyFunctionMap.IsEmpty());

	// Grab the blueprint variables
	NewClass->NumReplicatedProperties = 0;	// Keep track of how many replicated variables this blueprint adds

	const bool bRebuildPropertyMap = bIsFullCompile && !Blueprint->bIsRegeneratingOnLoad;
	if (bRebuildPropertyMap)
	{
		// Clear out any existing property guids if there could be local changes. The find code handles inherited variables
		NewClass->PropertyGuids.Reset();
	}
	NewClass->FieldNotifies.Reset();

	// Create variables in reverse order starting with NewVariables
	const int32 NumVariables = Blueprint->NewVariables.Num() + Blueprint->GeneratedVariables.Num();
	for (int32 i = 0; i < NumVariables; ++i)
	{
		const bool bIsGeneratedVariable = i >= Blueprint->NewVariables.Num();
		FBPVariableDescription& Variable = bIsGeneratedVariable
			? Blueprint->GeneratedVariables[Blueprint->GeneratedVariables.Num() - (i + 1 - Blueprint->NewVariables.Num())]
			: Blueprint->NewVariables[Blueprint->NewVariables.Num() - (i + 1)];

		FProperty* NewProperty = CreateVariable(Variable.VarName, Variable.VarType);
		if (NewProperty != NULL)
		{
			if(bAssignDelegateSignatureFunction)
			{
				if(FMulticastDelegateProperty* AsDelegate = CastField<FMulticastDelegateProperty>(NewProperty))
				{
					AsDelegate->SignatureFunction = FindUField<UFunction>(NewClass, *(Variable.VarName.ToString() + HEADER_GENERATED_DELEGATE_SIGNATURE_SUFFIX));
					// Skeleton compilation phase may run when the delegate has been created but the function has not:
					ensureAlways(AsDelegate->SignatureFunction || !bIsFullCompile);
				}
			}

			NewProperty->SetPropertyFlags((EPropertyFlags)Variable.PropertyFlags);
			NewProperty->SetMetaData(TEXT("DisplayName"), *Variable.FriendlyName);
			NewProperty->SetMetaData(TEXT("Category"), *Variable.Category.ToString());
			NewProperty->RepNotifyFunc = Variable.RepNotifyFunc;
			NewProperty->SetBlueprintReplicationCondition(Variable.ReplicationCondition);

			if (!NewProperty->RepNotifyFunc.IsNone())
			{
				if (TObjectPtr<UEdGraph>* Graph = Blueprint->FunctionGraphs.FindByPredicate([GraphName = Variable.RepNotifyFunc](UEdGraph* Other) { return Other->GetFName() == GraphName; }))
				{
					RepNotifyFunctionMap.Emplace(Variable.RepNotifyFunc, NewProperty);
				}
			}

			if(!Variable.DefaultValue.IsEmpty())
			{
				SetPropertyDefaultValue(NewProperty, Variable.DefaultValue);

				// We're copying the value to the real CDO, so clear the version stored in the blueprint editor data
				if (CompileOptions.CompileType == EKismetCompileType::Full)
				{
					Variable.DefaultValue.Empty();
				}
			}

			if (NewProperty->HasAnyPropertyFlags(CPF_Net))
			{
				NewClass->NumReplicatedProperties++;
			}

			// Set metadata on property
			for (FBPVariableMetaDataEntry& Entry : Variable.MetaDataArray)
			{
				NewProperty->SetMetaData(Entry.DataKey, *Entry.DataValue);
				if (Entry.DataKey == FBlueprintMetadata::MD_ExposeOnSpawn)
				{
					NewProperty->SetPropertyFlags(CPF_ExposeOnSpawn);
					if (NewProperty->HasAnyPropertyFlags(CPF_DisableEditOnInstance) && NewClass == Blueprint->SkeletonGeneratedClass)
					{
						MessageLog.Warning(
							*FText::Format(
								LOCTEXT("ExposeToSpawnButPrivateWarningFmt", "Variable {0} is marked as 'Expose on Spawn' but not marked as 'Instance Editable'; please make it 'Instance Editable'"),
								FText::FromName(NewProperty->GetFName())
							).ToString()
						);
					}
				}
			}

			bool bIsValidFieldNotify = Variable.HasMetaData(FBlueprintMetadata::MD_FieldNotify) && NewClass->ImplementsInterface(UNotifyFieldValueChanged::StaticClass());
			if (bIsValidFieldNotify)
			{
				ensure(!NewClass->FieldNotifies.Contains(FFieldNotificationId(NewProperty->GetFName())));
				NewClass->FieldNotifies.Add(FFieldNotificationId(NewProperty->GetFName()));
			}

			if (NewProperty->HasAnyPropertyFlags(CPF_Net) && NewProperty->RepNotifyFunc.IsNone() && bIsValidFieldNotify && NewClass == Blueprint->SkeletonGeneratedClass)
			{
				MessageLog.Warning(
					*FText::Format(
						LOCTEXT("ReplicatedFieldNotifyWarning", "Variable {0} is FieldNotify and replicated but not marked as 'RepNotify'; please make it 'RepNotify'."),
						FText::FromName(NewProperty->GetFName())
					).ToString()
				);
			}

			if (bRebuildPropertyMap)
			{
				// Update new class property guid map
				NewClass->PropertyGuids.Add(Variable.VarName, Variable.VarGuid);
			}
		}
	}

	// Ensure that timeline names are valid and that there are no collisions with a parent class
	ValidateTimelineNames();

	// Create a class property for each timeline instance contained in the blueprint
	for (int32 TimelineIndex = 0; TimelineIndex < Blueprint->Timelines.Num(); ++TimelineIndex)
	{
		UTimelineTemplate* Timeline = Blueprint->Timelines[TimelineIndex];
		// Not fatal if NULL, but shouldn't happen
		if (!Timeline)
		{
			continue;
		}

		FEdGraphPinType TimelinePinType(UEdGraphSchema_K2::PC_Object, NAME_None, UTimelineComponent::StaticClass(), EPinContainerType::None, false, FEdGraphTerminalType());

		// Previously UTimelineComponent object has exactly the same name as UTimelineTemplate object (that obj was in blueprint)
		const FName TimelineVariableName = Timeline->GetVariableName();
		if (FProperty* TimelineProperty = CreateVariable(TimelineVariableName, TimelinePinType))
		{
			FString CategoryName;
			if (Timeline->FindMetaDataEntryIndexForKey(TEXT("Category")) != INDEX_NONE)
			{
				CategoryName = Timeline->GetMetaData(TEXT("Category"));
			}
			else
			{
				CategoryName = Blueprint->GetName();
			}
			TimelineProperty->SetMetaData(TEXT("Category"), *CategoryName);
			TimelineProperty->SetPropertyFlags(CPF_BlueprintVisible);

			TimelineToMemberVariableMap.Add(Timeline, TimelineProperty);
		}

		FEdGraphPinType DirectionPinType(UEdGraphSchema_K2::PC_Byte, NAME_None, FTimeline::GetTimelineDirectionEnum(), EPinContainerType::None, false, FEdGraphTerminalType());
		CreateVariable(Timeline->GetDirectionPropertyName(), DirectionPinType);

		FEdGraphPinType FloatPinType(UEdGraphSchema_K2::PC_Real, UEdGraphSchema_K2::PC_Float, nullptr, EPinContainerType::None, false, FEdGraphTerminalType());

		for (const FTTFloatTrack& FloatTrack : Timeline->FloatTracks)
		{
			CreateVariable(FloatTrack.GetPropertyName(), FloatPinType);
		}

		FEdGraphPinType VectorPinType(UEdGraphSchema_K2::PC_Struct, NAME_None, VectorStruct, EPinContainerType::None, false, FEdGraphTerminalType());
		for (const FTTVectorTrack& VectorTrack : Timeline->VectorTracks)
		{
			CreateVariable(VectorTrack.GetPropertyName(), VectorPinType);
		}

		FEdGraphPinType LinearColorPinType(UEdGraphSchema_K2::PC_Struct, NAME_None, LinearColorStruct, EPinContainerType::None, false, FEdGraphTerminalType());
		for (const FTTLinearColorTrack& LinearColorTrack : Timeline->LinearColorTracks)
		{
			CreateVariable(LinearColorTrack.GetPropertyName(), LinearColorPinType);
		}
	}

	// Create a class property for any simple-construction-script created components that should be exposed
	if (Blueprint->SimpleConstructionScript)
	{
		// Ensure that nodes have valid templates (This will remove nodes that have had the classes the inherited from removed
		Blueprint->SimpleConstructionScript->ValidateNodeTemplates(MessageLog);

		// Ensure that variable names are valid and that there are no collisions with a parent class
		Blueprint->SimpleConstructionScript->ValidateNodeVariableNames(MessageLog);

		for (USCS_Node* Node : Blueprint->SimpleConstructionScript->GetAllNodes())
		{
			if (Node)
			{
				FName VarName = Node->GetVariableName();
				if ((VarName != NAME_None) && (Node->ComponentClass != nullptr))
				{
					FEdGraphPinType Type(UEdGraphSchema_K2::PC_Object, NAME_None, Node->ComponentClass, EPinContainerType::None, false, FEdGraphTerminalType());
					if (FProperty* NewProperty = CreateVariable(VarName, Type))
					{
						const FText CategoryName = Node->CategoryName.IsEmpty() ? FText::FromString(Blueprint->GetName()) : Node->CategoryName ;
					
						NewProperty->SetMetaData(TEXT("Category"), *CategoryName.ToString());
						NewProperty->SetPropertyFlags(CPF_BlueprintVisible | CPF_NonTransactional );
					}
				}
			}
		}
	}
}

void FKismetCompilerContext::CreatePropertiesFromList(UStruct* Scope, FField::FLinkedListBuilder& PropertyListBuilder, TIndirectArray<FBPTerminal>& Terms, EPropertyFlags PropertyFlags, bool bPropertiesAreLocal, bool bPropertiesAreParameters)
{
	for (FBPTerminal& Term : Terms)
	{
		if (Term.AssociatedVarProperty)
		{
			if(Term.Context && !Term.Context->IsObjectContextType())
			{
				continue;
			}
			MessageLog.Warning(
				*FText::Format(
					LOCTEXT("AssociatedVarProperty_ErrorFmt", "AssociatedVarProperty property overridden {0} from @@ type ({1})"),
					FText::FromString(Term.Name),
					UEdGraphSchema_K2::TypeToText(Term.Type)
				).ToString(),
				Term.Source
			);
		}

		if (Term.bIsLiteral)
		{
			MessageLog.Error(
				*FText::Format(
					LOCTEXT("PropertyForLiteral_ErrorFmt", "Cannot create property for a literal: {0} from @@ type ({1})"),
					FText::FromString(Term.Name),
					UEdGraphSchema_K2::TypeToText(Term.Type)
				).ToString(),
				Term.Source
			);
		}

		if (FProperty* NewProperty = FKismetCompilerUtilities::CreatePropertyOnScope(Scope, FName(*Term.Name), Term.Type, NewClass, PropertyFlags, Schema, MessageLog, Term.SourcePin))
		{
			if (bPropertiesAreParameters && Term.Type.bIsConst)
			{
				NewProperty->SetPropertyFlags(CPF_ConstParm);
			}

			if (Term.bPassedByReference)
			{
				// special case for BlueprintImplementableEvent
				if (NewProperty->HasAnyPropertyFlags(CPF_Parm) && !NewProperty->HasAnyPropertyFlags(CPF_OutParm))
				{
					NewProperty->SetPropertyFlags(CPF_OutParm | CPF_ReferenceParm);
				}
			}

			if (Term.bIsSavePersistent)
			{
				NewProperty->SetPropertyFlags(CPF_SaveGame);
			}

			// Imply read only for input object pointer parameters to a const class
			//@TODO: UCREMOVAL: This should really happen much sooner, and isn't working here
			if (bPropertiesAreParameters && ((PropertyFlags & CPF_OutParm) == 0))
			{
				if (FObjectProperty* ObjProp = CastField<FObjectProperty>(NewProperty))
				{
					UClass* EffectiveClass = NULL;
					if (ObjProp->PropertyClass != NULL)
					{
						EffectiveClass = ObjProp->PropertyClass;
					}
					else if (FClassProperty* ClassProp = CastField<FClassProperty>(ObjProp))
					{
						EffectiveClass = ClassProp->MetaClass;
					}


					if ((EffectiveClass != NULL) && (EffectiveClass->HasAnyClassFlags(CLASS_Const)))
					{
						NewProperty->PropertyFlags |= CPF_ConstParm;
					}
				}
				else if (FArrayProperty* ArrayProp = CastField<FArrayProperty>(NewProperty))
				{
					NewProperty->PropertyFlags |= CPF_ReferenceParm;

					// ALWAYS pass array parameters as out params, so they're set up as passed by ref
					if( (PropertyFlags & CPF_Parm) != 0 )
					{
						NewProperty->PropertyFlags |= CPF_OutParm;
					}
				}
			}
			
			// Link this object to the tail of the list (so properties remain in the desired order)
			PropertyListBuilder.AppendNoTerminate(*NewProperty);

			Term.AssociatedVarProperty = NewProperty;
			Term.SetVarTypeLocal(bPropertiesAreLocal);

			// Record in the debugging information
			//@TODO: Rename RegisterClassPropertyAssociation, etc..., to better match that indicate it works with locals
			{
				if (Term.SourcePin)
				{
					UEdGraphPin* TrueSourcePin = MessageLog.FindSourcePin(Term.SourcePin);
					NewClass->GetDebugData().RegisterClassPropertyAssociation(TrueSourcePin, NewProperty);
				}
				else
				{
					UObject* TrueSourceObject = MessageLog.FindSourceObject(Term.Source);
					NewClass->GetDebugData().RegisterClassPropertyAssociation(TrueSourceObject, NewProperty);
				}
			}

			// Record the desired default value for this, if specified by the term
			if (!Term.PropertyDefault.IsEmpty())
			{
				if (bPropertiesAreParameters)
				{
					const bool bInputParameter = (0 == (PropertyFlags & CPF_OutParm)) && (0 != (PropertyFlags & CPF_Parm));
					if (bInputParameter)
					{
						Scope->SetMetaData(NewProperty->GetFName(), *Term.PropertyDefault);
					}
					else
					{
						MessageLog.Warning(
							*FText::Format(LOCTEXT("UnusedDefaultValue_WarnFmt", "Default value for '{0}' cannot be used."), FText::FromString(NewProperty->GetName())).ToString(),
							Term.Source);
					}
				}
				else
				{
					SetPropertyDefaultValue(NewProperty, Term.PropertyDefault);
				}
			}
		}
		else
		{
			MessageLog.Error(
				*FText::Format(
					LOCTEXT("FailedCreateProperty_ErrorFmt", "Failed to create property {0} from @@ due to a bad or unknown type ({1})"),
					FText::FromString(Term.Name),
					UEdGraphSchema_K2::TypeToText(Term.Type)
				).ToString(),
				Term.Source
			);
		}

	}
}

void FKismetCompilerContext::CreateParametersForFunction(FKismetFunctionContext& Context, UFunction* ParameterSignature, FField::FLinkedListBuilder& PropertyListBuilder)
{
	const bool bArePropertiesLocal = true;
	CreatePropertiesFromList(Context.Function, PropertyListBuilder, Context.Parameters, CPF_Parm | CPF_BlueprintVisible | CPF_BlueprintReadOnly, bArePropertiesLocal, /*bPropertiesAreParameters=*/ true);
	CreatePropertiesFromList(Context.Function, PropertyListBuilder, Context.Results, CPF_Parm | CPF_OutParm, bArePropertiesLocal, /*bPropertiesAreParameters=*/ true);

	//MAKE SURE THE PARAMETERS ORDER MATCHES THE OVERRIDEN FUNCTION
	if (ParameterSignature)
	{
		// copy all fields before reordering them
		TArray<FField*> SortedFields;
		for (FField* FieldIt = PropertyListBuilder.GetListStart(); FieldIt; FieldIt = PropertyListBuilder.GetNext(*FieldIt))
		{
			SortedFields.Add(FieldIt);
		}

		// sort to match signature using property names
		int32 SortFieldIdx = 0;
		for (TFieldIterator<FProperty> SignatureIt(ParameterSignature); SignatureIt && ((SignatureIt->PropertyFlags & CPF_Parm) != 0); ++SignatureIt, ++SortFieldIdx)
		{
			const FName WantedName = SignatureIt->GetFName();
			bool bFoundMatch = false;

			if (SortFieldIdx < SortedFields.Num())
			{
				FField* SortFieldIt = SortedFields[SortFieldIdx];
				if (SortFieldIt->GetFName() != WantedName)
				{
					for (int32 MatchIdx = SortFieldIdx + 1; MatchIdx < SortedFields.Num(); MatchIdx++)
					{
						if (SortedFields[MatchIdx]->GetFName() == WantedName)
						{
							SortedFields.Swap(MatchIdx, SortFieldIdx);
							bFoundMatch = true;
							break;
						}
					}
				}
				else
				{
					bFoundMatch = true;
				}
			}

			if (bFoundMatch)
			{
				// Ensure that the 'CPF_UObjectWrapper' flag is propagated through to new parameters, so that wrapper types like 'TSubclassOf' can be preserved if the compiled UFunction is ever nativized.
				if (SignatureIt->HasAllPropertyFlags(CPF_UObjectWrapper))
				{
					CastFieldChecked<FProperty>(SortedFields[SortFieldIdx])->SetPropertyFlags(CPF_UObjectWrapper);
				}
			}
			else
			{
				MessageLog.Error(
					*FText::Format(
						LOCTEXT("WrongParameterOrder_ErrorFmt", "Cannot order parameters {0} in function {1}."),
						FText::FromName(WantedName),
						FText::FromString(Context.Function->GetName())
					).ToString()
				);
				break;
			}
		}

		// rebuild list with new order
		PropertyListBuilder.Restart();
		for (int32 Idx = 0; Idx < SortedFields.Num(); Idx++)
		{
			PropertyListBuilder.AppendNoTerminate(*SortedFields[Idx]);
		}
		PropertyListBuilder.NullTerminate();
	}
}

void FKismetCompilerContext::CreateLocalVariablesForFunction(FKismetFunctionContext& Context, FField::FLinkedListBuilder& PropertyListBuilder)
{
	ensure(Context.IsEventGraph() || !Context.EventGraphLocals.Num());
	ensure(!Context.IsEventGraph() || !Context.Locals.Num() || !UsePersistentUberGraphFrame());

	const bool bPersistentUberGraphFrame = UsePersistentUberGraphFrame() && Context.bIsUbergraph;
	// Local stack frame (or maybe class for the ubergraph)
	{
		const bool bArePropertiesLocal = true;

		CreatePropertiesFromList(Context.Function, PropertyListBuilder, Context.Locals, CPF_None, bArePropertiesLocal, /*bPropertiesAreParameters=*/ true);

		if (bPersistentUberGraphFrame)
		{
			CreatePropertiesFromList(Context.Function, PropertyListBuilder, Context.EventGraphLocals, CPF_None, bArePropertiesLocal, true);
		}

		// Create debug data for variable reads/writes
		if (Context.bCreateDebugData)
		{
			for (int32 VarAccessIndex = 0; VarAccessIndex < Context.VariableReferences.Num(); ++VarAccessIndex)
			{
				FBPTerminal& Term = Context.VariableReferences[VarAccessIndex];

				if (Term.AssociatedVarProperty != NULL)
				{
					if (Term.SourcePin)
					{
						UEdGraphPin* TrueSourcePin = MessageLog.FindSourcePin(Term.SourcePin);
						NewClass->GetDebugData().RegisterClassPropertyAssociation(TrueSourcePin, Term.AssociatedVarProperty);
					}
					else
					{
						UObject* TrueSourceObject = MessageLog.FindSourceObject(Term.Source);
						NewClass->GetDebugData().RegisterClassPropertyAssociation(TrueSourceObject, Term.AssociatedVarProperty);
					}
				}
			}
		}

		// Fix up the return value
		//@todo:  Is there a better way of doing this without mangling code?
		for (TFieldIterator<FProperty> It(Context.Function); It && (It->PropertyFlags & CPF_Parm); ++It)
		{
			FProperty* Property = *It;
			if ((Property->GetFName() == UEdGraphSchema_K2::PN_ReturnValue) && Property->HasAnyPropertyFlags(CPF_OutParm))
			{
				Property->SetPropertyFlags(CPF_ReturnParm);
			}
		}
	}

	// Class
	{
		int32 PropertySafetyCounter = 100000;
		FField::FLinkedListBuilder ClassPropertyListBuilder(&NewClass->ChildProperties);

		while (ClassPropertyListBuilder.MoveToNext())
		{
			if (--PropertySafetyCounter == 0)
			{
				checkf(false, TEXT("Property chain is corrupted;  The most likely causes are multiple properties with the same name.") );
			}
		}

		const bool bArePropertiesLocal = false;
		const EPropertyFlags UbergraphHiddenVarFlags = CPF_Transient | CPF_DuplicateTransient;
		if (!bPersistentUberGraphFrame)
		{
			CreatePropertiesFromList(NewClass, ClassPropertyListBuilder, Context.EventGraphLocals, UbergraphHiddenVarFlags, bArePropertiesLocal);
		}

		// Handle level actor references
		const EPropertyFlags LevelActorReferenceVarFlags = CPF_None/*CPF_Edit*/;
		CreatePropertiesFromList(NewClass, ClassPropertyListBuilder, Context.LevelActorReferences, LevelActorReferenceVarFlags, false);
	}
}

void FKismetCompilerContext::CreateUserDefinedLocalVariablesForFunction(FKismetFunctionContext& Context, FField::FLinkedListBuilder& PropertyListBuilder)
{
	// Create local variables from the Context entry point
	for (int32 i = 0; i < Context.EntryPoint->LocalVariables.Num(); ++i)
	{
		FBPVariableDescription& Variable = Context.EntryPoint->LocalVariables[Context.EntryPoint->LocalVariables.Num() - (i + 1)];
		FProperty* NewProperty = CreateUserDefinedLocalVariableForFunction(Variable, Context.Function, NewClass, PropertyListBuilder, Schema, MessageLog);

		if (NewProperty != NULL)
		{
			if(!Variable.DefaultValue.IsEmpty())
			{
				SetPropertyDefaultValue(NewProperty, Variable.DefaultValue);
			}
		}
	}
}

FProperty* FKismetCompilerContext::CreateUserDefinedLocalVariableForFunction(const FBPVariableDescription& Variable, UFunction* Function, UBlueprintGeneratedClass* OwningClass, FField::FLinkedListBuilder& PropertyListBuilder, const UEdGraphSchema_K2* Schema, FCompilerResultsLog& MessageLog)
{
	FProperty* NewProperty = FKismetCompilerUtilities::CreatePropertyOnScope(Function, Variable.VarName, Variable.VarType, OwningClass, CPF_None, Schema, MessageLog);
	
	if(NewProperty)
	{
		// Link this object to the tail of the list (so properties remain in the desired order)
		PropertyListBuilder.AppendNoTerminate(*NewProperty);
		
		NewProperty->SetPropertyFlags((EPropertyFlags)Variable.PropertyFlags);
		NewProperty->SetMetaData(TEXT("FriendlyName"), *Variable.FriendlyName);
		NewProperty->SetMetaData(TEXT("Category"), *Variable.Category.ToString());
		NewProperty->RepNotifyFunc = Variable.RepNotifyFunc;
		NewProperty->SetPropertyFlags((EPropertyFlags)Variable.PropertyFlags);

		// Set metadata on property (some customizations may expose it)
		for (const FBPVariableMetaDataEntry& Entry : Variable.MetaDataArray)
		{
			NewProperty->SetMetaData(Entry.DataKey, *Entry.DataValue);
		}
	}
	
	return NewProperty;
}

void FKismetCompilerContext::SetPropertyDefaultValue(const FProperty* PropertyToSet, FString& Value)
{
	DefaultPropertyValueMap.Add(PropertyToSet->GetFName(), Value);
}

/** Copies default values cached for the terms in the DefaultPropertyValueMap to the final CDO */
void FKismetCompilerContext::CopyTermDefaultsToDefaultObject(UObject* DefaultObject)
{
	if (DefaultPropertyValueMap.Num() > 0)
	{
		int32 PropertiesAssigned = 0;

		// Assign all default object values from the map to the new CDO
		for (TFieldIterator<FProperty> It(DefaultObject->GetClass(), EFieldIteratorFlags::ExcludeSuper); It && PropertiesAssigned < DefaultPropertyValueMap.Num(); ++It)
		{
			FProperty* Property = *It;
			if (const FString* ValuePtr = DefaultPropertyValueMap.Find(Property->GetFName()))
			{
				++PropertiesAssigned;

				const FString& Value = *ValuePtr;
				if (FObjectProperty* AsObjectProperty = CastField<FObjectProperty>(Property))
				{
					// Value is the fully qualified name, so just search for it:
					UObject* Result = StaticFindObjectSafe(UObject::StaticClass(), nullptr, *Value);
					if (Result)
					{
						// Object may be of a type that is also being compiled and therefore REINST_, so get real class:
						UClass* RealClass = Result->GetClass()->GetAuthoritativeClass();

						// If object is compatible, write it into cdo:
						if (RealClass && RealClass->IsChildOf(AsObjectProperty->PropertyClass))
						{
							AsObjectProperty->SetObjectPropertyValue(AsObjectProperty->ContainerPtrToValuePtr<uint8>(DefaultObject), Result);
							continue;
						}
					}
				}

				// If this property contains an instanced reference, set the flag to make sure we uniquely instance those objects when we import the default value.
				int32 PortFlags = PPF_None;
				if (Property->HasAnyPropertyFlags(CPF_ContainsInstancedReference | CPF_InstancedReference))
				{
					PortFlags |= PPF_InstanceSubobjects;
				}

				const bool bParseSucceeded = FBlueprintEditorUtils::PropertyValueFromString(Property, Value, reinterpret_cast<uint8*>(DefaultObject), DefaultObject, PortFlags);
				if(!bParseSucceeded)
				{
					const FString ErrorMessage = *FText::Format(
						LOCTEXT("ParseDefaultValueErrorFmt", "Can't parse default value '{0}' for @@. Property: {1}."),
						FText::FromString(Value),
						FText::FromString(Property->GetName())
					).ToString();
					UObject* InstigatorObject = NewClass->GetDebugData().FindObjectThatCreatedProperty(Property);
					if(InstigatorObject)
					{
						MessageLog.Warning(*ErrorMessage, InstigatorObject);
					}
					else
					{
						UEdGraphPin* InstigatorPin = NewClass->GetDebugData().FindPinThatCreatedProperty(Property);
						MessageLog.Warning(*ErrorMessage, InstigatorPin);
					}
				}
			}
		}			
	}
}

void FKismetCompilerContext::PropagateValuesToCDO(UObject* InNewCDO, UObject* InOldCDO)
{
	ensure(InNewCDO);
	CopyTermDefaultsToDefaultObject(InNewCDO);
	SetCanEverTick();
}

void FKismetCompilerContext::PrintVerboseInfoStruct(UStruct* Struct) const
{
	for (TFieldIterator<FProperty> PropIt(Struct); PropIt; ++PropIt)
	{
		FProperty* Prop = *PropIt;

		MessageLog.Note(
			*FText::Format(
				LOCTEXT("StructInfo_NoteFmt", "  {0} named {1} at offset {2} with size {3} [dim = {4}] and flags {5}"),
				FText::FromString(Prop->GetClass()->GetDescription()),
				FText::FromString(Prop->GetName()),
				Prop->GetOffset_ForDebug(),
				Prop->GetElementSize(),
				Prop->ArrayDim,
				FText::FromString(FString::Printf(TEXT("%" UINT64_x_FMT), (uint64)Prop->PropertyFlags))
			).ToString()
		);
	}
}

void FKismetCompilerContext::PrintVerboseInformation(UClass* Class) const
{
	MessageLog.Note(
		*FText::Format(
			LOCTEXT("ClassHasMembers_NoteFmt", "Class {0} has members:"),
			FText::FromString(Class->GetName())
		).ToString()
	);
	PrintVerboseInfoStruct(Class);

	for (int32 i = 0; i < FunctionList.Num(); ++i)
	{
		const FKismetFunctionContext& Context = FunctionList[i];

		if (Context.IsValid())
		{
			MessageLog.Note(*FText::Format(LOCTEXT("FunctionHasMembers_NoteFmt", "Function {0} has members:"), FText::FromString(Context.Function->GetName())).ToString());
			PrintVerboseInfoStruct(Context.Function);
		}
		else
		{
			MessageLog.Note(*FText::Format(LOCTEXT("FunctionCompileFailed_NoteFmt", "Function #{0} failed to compile and is not valid."), i).ToString());
		}
	}
}

void FKismetCompilerContext::CheckConnectionResponse(const FPinConnectionResponse &Response, const UEdGraphNode *Node)
{
	if (!Response.CanSafeConnect())
	{
		MessageLog.Error(*FText::Format(LOCTEXT("FailedBuildingConnection_ErrorFmt", "COMPILER ERROR: failed building connection with '{0}' at @@"), Response.Message).ToString(), Node);
	}
}

/**
 * Performs transformations on specific nodes that require it according to the schema
 */
void FKismetCompilerContext::TransformNodes(FKismetFunctionContext& Context)
{
	// Give every node a chance to transform itself
	for (int32 NodeIndex = 0; NodeIndex < Context.SourceGraph->Nodes.Num(); ++NodeIndex)
	{
		UEdGraphNode* Node = Context.SourceGraph->Nodes[NodeIndex];

		if (FNodeHandlingFunctor* Handler = NodeHandlers.FindRef(Node->GetClass()))
		{
			Handler->Transform(Context, Node);
		}
		else
		{
			MessageLog.Error(*FText::Format(LOCTEXT("UnexpectedNodeType_ErrorFmt", "Unexpected node type {0} encountered at @@"), FText::FromString(Node->GetClass()->GetName())).ToString(), Node);
		}
	}
}

/** Use to traverse exec wires to identify impure (exec) nodes that are used (and shouldn't be pruned) */
struct FNodeVisitorDownExecWires
{
	TSet<UEdGraphNode*> VisitedNodes;
	UEdGraphSchema_K2* Schema;

	void TouchNode(UEdGraphNode* Node)
	{
	}

	void TraverseNodes(UEdGraphNode* Node)
	{
		VisitedNodes.Add(Node);
		TouchNode(Node);

		// Follow every exec output pin
		for (int32 i = 0; i < Node->Pins.Num(); ++i)
		{
			UEdGraphPin* MyPin = Node->Pins[i];

			if ((MyPin->Direction == EGPD_Output) && (Schema->IsExecPin(*MyPin)))
			{
				for (int32 j = 0; j < MyPin->LinkedTo.Num(); ++j)
				{
					UEdGraphPin* OtherPin = MyPin->LinkedTo[j];
					if( OtherPin )
					{
						UEdGraphNode* OtherNode = OtherPin->GetOwningNode();
						if (!VisitedNodes.Contains(OtherNode))
						{
							TraverseNodes(OtherNode);
						}
					}
				}
			}
		}
	}
};

/** Use to traverse data wires (out from exec nodes) to identify pure nodes that are used (and shouldn't be pruned) */
struct FNodeVisitorUpDataWires
{
	TSet<UEdGraphNode*> VisitedNodes;
	UEdGraphSchema_K2* Schema;

	void TraverseNodes(UEdGraphNode* Node)
	{
		bool bAlreadyVisited = false;
		VisitedNodes.Add(Node, &bAlreadyVisited);
		if (!bAlreadyVisited)
		{
			// Follow every data input pin
			// we don't have to worry about unconnected non-pure nodes, thay were already removed
			// we want to gather all pure nodes, that are really used
			for (int32 i = 0; i < Node->Pins.Num(); ++i)
			{
				UEdGraphPin* MyPin = Node->Pins[i];

				if ((MyPin->Direction == EGPD_Input) && !Schema->IsExecPin(*MyPin))
				{
					for (int32 j = 0; j < MyPin->LinkedTo.Num(); ++j)
					{
						UEdGraphPin* OtherPin = MyPin->LinkedTo[j];
						if (OtherPin)
						{
							UEdGraphNode* OtherNode = OtherPin->GetOwningNode();
							if (!VisitedNodes.Contains(OtherNode))
							{
								TraverseNodes(OtherNode);
							}
						}
					}
				}
			}
		}
	}
};

bool FKismetCompilerContext::CanIgnoreNode(const UEdGraphNode* Node) const
{
	if (const UK2Node* K2Node = Cast<const UK2Node>(Node))
	{
		return K2Node->IsNodeSafeToIgnore();
	}
	return false;
}

bool FKismetCompilerContext::ShouldForceKeepNode(const UEdGraphNode* Node) const
{
	if (Node->IsA(UEdGraphNode_Comment::StaticClass()) && CompileOptions.bSaveIntermediateProducts)
	{
		// Preserve comment nodes when debugging the compiler
		return true;
	}
	else
	{
		return false;
	}
}

void FKismetCompilerContext::PruneIsolatedNodes(UEdGraph* InGraph, bool bInIncludeNodesThatCouldBeExpandedToRootSet)
{
	TArray<UEdGraphNode*> RootSet;
	// Find any all entry points caused by special nodes
	UE::KismetCompiler::Private::GatherRootSet(InGraph, RootSet, bInIncludeNodesThatCouldBeExpandedToRootSet);

	// Find the connected subgraph starting at the root node and prune out unused nodes
	PruneIsolatedNodes(RootSet, MutableView(InGraph->Nodes));
}

/** Prunes any nodes that weren't visited from the graph, printing out a warning */
void FKismetCompilerContext::PruneIsolatedNodes(const TArray<UEdGraphNode*>& RootSet, TArray<UEdGraphNode*>& GraphNodes)
{
	BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_PruneIsolatedNodes);
	
	//@TODO: This function crawls the graph twice (once here and once in Super, could potentially combine them, with a bitflag for flows reached via exec wires)

	// Prune the impure nodes that aren't reachable via any (even impossible, e.g., a branch never taken) execution flow
	FNodeVisitorDownExecWires Visitor;
	Visitor.Schema = Schema;

	for (TArray<UEdGraphNode*>::TConstIterator It(RootSet); It; ++It)
	{
		UEdGraphNode* RootNode = *It;
		Visitor.TraverseNodes(RootNode);
	}

	const UEdGraphSchema* const K2Schema = UEdGraphSchema_K2::StaticClass()->GetDefaultObject<UEdGraphSchema_K2>();
	TMap<UEdGraphNode*, TArray<UEdGraphNode*>> PrunedExecNodeNeighbors;
	for (int32 NodeIndex = 0; NodeIndex < GraphNodes.Num(); ++NodeIndex)
	{
		UEdGraphNode* Node = GraphNodes[NodeIndex];
		if (!Node || (!Visitor.VisitedNodes.Contains(Node) && !IsNodePure(Node)))
		{
			auto ShouldKeepNonPureNodeWithoutExecPin = [&]() -> bool
			{
				if (Node 
					&& Node->CanCreateUnderSpecifiedSchema(K2Schema) // Anim Nodes still should be pruned
					&& !Node->IsA<UK2Node_Tunnel>()) // Tunnels are never pure.
				{
					bool bHasExecPin = false;
					for (const UEdGraphPin* Pin : Node->Pins)
					{
						if (Pin && (Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec))
						{
							bHasExecPin = true;
							break;
						}
					}
					if (!bHasExecPin)
					{
						const FString WarningStr = FText::Format(LOCTEXT("NoPureNodeWithoutExec_WarningFmt", "Node @@. The node won't be pruned as isolated one. The node is not pure, but it has no exec pin(s). Verify IsNodePure implementation in {0}."), Node->GetClass()->GetDisplayNameText()).ToString();
						MessageLog.Warning(*WarningStr, Node);
					}
					return !bHasExecPin;
				}
				return false;
			};

			if (!Node || (!ShouldForceKeepNode(Node) && !ShouldKeepNonPureNodeWithoutExecPin()))
			{
				if (Node)
				{
					// Track nodes that are directly connected to the outputs of the node we are pruning so 
					// that we can warn if one or more of those neighboring nodes are not also orphaned:
					Node->ForEachNodeDirectlyConnectedIf(
						// Consider connections on output pins other than the exec pin:
						[](const UEdGraphPin* Pin) {
							if(Pin->Direction == EGPD_Output && Pin->PinType.PinCategory != UEdGraphSchema_K2::PC_Exec) 
							{ 
								return true; 
							}
							return false;
						},
						[&PrunedExecNodeNeighbors, Node](UEdGraphNode* NeighborNode) { PrunedExecNodeNeighbors.FindOrAdd(Node).Add(NeighborNode); }
					);
					Node->BreakAllNodeLinks();
				}
				GraphNodes.RemoveAtSwap(NodeIndex);
				--NodeIndex;
			}
		}
	}

	// Prune the nodes that aren't even reachable via data dependencies
	Super::PruneIsolatedNodes(RootSet, GraphNodes);

	{
		FNodeVisitorUpDataWires UpDataVisitor;
		UpDataVisitor.Schema = Schema;
		// we still have pure nodes that could afford to be pruned, so let's 
		// explore data wires (from the impure nodes we kept), and identify
		// pure nodes we want to keep
		for (UEdGraphNode* VisitedNode : Visitor.VisitedNodes)
		{
			UK2Node* K2Node = Cast<UK2Node>(VisitedNode);
			if (K2Node && !K2Node->IsNodePure())
			{
				UpDataVisitor.TraverseNodes(VisitedNode);
			}
		}

		// remove pure nodes that are unused (ones that weren't visited by traversing data wires)
		for (int32 NodeIndex = 0; NodeIndex < GraphNodes.Num(); ++NodeIndex)
		{
			UK2Node* K2Node = Cast<UK2Node>(GraphNodes[NodeIndex]);
			if (K2Node && K2Node->IsNodePure() && !UpDataVisitor.VisitedNodes.Contains(K2Node) 
				&& !K2Node->IsA<UK2Node_Knot>()) // Knots are pure, but they can have exec pins
			{
				if (!ShouldForceKeepNode(K2Node))
				{
					K2Node->BreakAllNodeLinks();
					GraphNodes.RemoveAtSwap(NodeIndex);
					--NodeIndex;
				}
			}
		}
	}

	for(const TPair<UEdGraphNode*, TArray<UEdGraphNode*>>& PrunedExecNodeWithNeighbors : PrunedExecNodeNeighbors)
	{
		bool bNeighborsNotPruned = false;
		for(UEdGraphNode* Neighbor : PrunedExecNodeWithNeighbors.Value)
		{
			if(GraphNodes.Contains(Neighbor))
			{
				bNeighborsNotPruned = true;
			}
		}

		if(bNeighborsNotPruned)
		{
			// Warn the user if they are attempting to read an output value from a pruned exec node:
			MessageLog.Warning(FName(TEXT("PrunedExecInUse")), *LOCTEXT("PrunedExecNodeAttemptedUse", "@@ was pruned because its Exec pin is not connected, the connected value is not available and will instead be read as default").ToString(), PrunedExecNodeWithNeighbors.Key);
		}
	}
}

/**
 *	Checks if self pins are connected.
 */
void FKismetCompilerContext::ValidateSelfPinsInGraph(FKismetFunctionContext& Context)
{
	const UEdGraph* SourceGraph = Context.SourceGraph;

	check(NULL != Schema);
	for (int32 NodeIndex = 0; NodeIndex < SourceGraph->Nodes.Num(); ++NodeIndex)
	{
		if(const UEdGraphNode* Node = SourceGraph->Nodes[NodeIndex])
		{
			for (int32 PinIndex = 0; PinIndex < Node->Pins.Num(); ++PinIndex)
			{
				if (const UEdGraphPin* Pin = Node->Pins[PinIndex])
				{
					if (Schema->IsSelfPin(*Pin) && (Pin->LinkedTo.Num() == 0) && Pin->DefaultObject == nullptr)
					{
						FKismetCompilerUtilities::ValidateSelfCompatibility(Pin, Context);
					}
				}
			}
		}
	}
}

void FKismetCompilerContext::ValidateNoWildcardPinsInGraph(const UEdGraph* SourceGraph)
{
	for (int32 NodeIndex = 0; NodeIndex < SourceGraph->Nodes.Num(); ++NodeIndex)
	{
		if (const UEdGraphNode* Node = SourceGraph->Nodes[NodeIndex])
		{
			for (int32 PinIndex = 0; PinIndex < Node->Pins.Num(); ++PinIndex)
			{
				if (const UEdGraphPin* Pin = Node->Pins[PinIndex])
				{
					if (Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard)
					{
						// Wildcard pins should never be seen by the compiler; they should always be forced into a particular type by wiring.
						MessageLog.Error(*LOCTEXT("UndeterminedPinType_Error", "The type of @@ is undetermined.  Connect something to @@ to imply a specific type.").ToString(), Pin, Pin->GetOwningNodeUnchecked());
					}
				}
			}
		}
	}
}

/**
 * First phase of compiling a function graph
 *   - Prunes the 'graph' to only included the connected portion that contains the function entry point 
 *   - Schedules execution of each node based on data dependencies
 *   - Creates a UFunction object containing parameters and local variables (but no script code yet)
 */
void FKismetCompilerContext::PrecompileFunction(FKismetFunctionContext& Context, EInternalCompilerFlags InternalFlags)
{
	BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_PrecompileFunction);

	const bool bImmediatelyGenerateLocals = !(InternalFlags & EInternalCompilerFlags::PostponeLocalsGenerationUntilPhaseTwo);

	// Find the root node, which will drive everything else
	TArray<UK2Node_FunctionEntry*> EntryPoints;
	Context.SourceGraph->GetNodesOfClass(EntryPoints);

	if (EntryPoints.Num())
	{
		Context.EntryPoint = EntryPoints[0];

		// Make sure there was only one function entry node
		for (int32 i = 1; i < EntryPoints.Num(); ++i)
		{
			MessageLog.Error(
				*LOCTEXT("ExpectedOneFunctionEntry_Error", "Expected only one function entry node in graph @@, but found both @@ and @@").ToString(),
				Context.SourceGraph,
				Context.EntryPoint,
				EntryPoints[i]
			);
		}

		// Find the connected subgraph starting at the root node and prune out unused nodes
		const bool bIncludePotentialRootNodes = false;
		PruneIsolatedNodes(Context.SourceGraph, bIncludePotentialRootNodes);

		if (bIsFullCompile)
		{
			// Check if self pins are connected and types are resolved after PruneIsolatedNodes, to avoid errors from isolated nodes.
			ValidateSelfPinsInGraph(Context);
			ValidateNoWildcardPinsInGraph(Context.SourceGraph);

			// Transforms
			TransformNodes(Context);
		}

		// Create the function stub
		FName NewFunctionName = (Context.EntryPoint->CustomGeneratedFunctionName != NAME_None) ? Context.EntryPoint->CustomGeneratedFunctionName : Context.EntryPoint->FunctionReference.GetMemberName();
		if (Context.IsDelegateSignature())
		{
			// prefix with the the blueprint name to avoid conflicts with natively defined delegate signatures
			FString Name = NewFunctionName.ToString();
			Name += HEADER_GENERATED_DELEGATE_SIGNATURE_SUFFIX;
			NewFunctionName = FName(*Name);
		}

		// Determine if this is a new function or if it overrides a parent function
			//@TODO: Does not support multiple overloads for a parent virtual function
		UClass* SuperClass = Context.NewClass->GetSuperClass();
		UFunction* ParentFunction = Context.NewClass->GetSuperClass()->FindFunctionByName(NewFunctionName);
		
		const FString NewFunctionNameString = NewFunctionName.ToString();
		if (CreatedFunctionNames.Contains(NewFunctionNameString))
		{
			MessageLog.Error(
				*FText::Format(
					LOCTEXT("DuplicateFunctionName_ErrorFmt", "Found more than one function with the same name {0}; second occurance at @@"),
					FText::FromString(NewFunctionNameString)
				).ToString(),
				Context.EntryPoint
			);
			return;
		}
		else if (NULL != FindFProperty<FProperty>(NewClass, NewFunctionName))
		{
			MessageLog.Error(
				*FText::Format(
					LOCTEXT("DuplicateFieldName_ErrorFmt", "Name collision - function and property have the same name - '{0}'. @@"),
					FText::FromString(NewFunctionNameString)
				).ToString(),
				Context.EntryPoint
			);
			return;
		}
		else
		{
			CreatedFunctionNames.Add(NewFunctionNameString);
		}

		Context.Function = NewObject<UFunction>(NewClass, NewFunctionName, RF_Public);

		if( UE::Private::GCanSaveFunctionsForPackage && 
			!(*UE::Private::GCanSaveFunctionsForPackage)(Blueprint->GetPackage()))
		{
			Context.Function->SetFlags(RF_Transient);
		}
#if USE_TRANSIENT_SKELETON
		// Propagate down transient settings from the class
		if (NewClass->HasAnyFlags(RF_Transient))
		{
			Context.Function->SetFlags(RF_Transient);
		}
#endif

		Context.Function->SetSuperStruct( ParentFunction );
		Context.Function->ReturnValueOffset = MAX_uint16;
		Context.Function->FirstPropertyToInit = NULL;

		// Set up the function category
		FKismetUserDeclaredFunctionMetadata& FunctionMetaData = Context.EntryPoint->MetaData;
		if (!FunctionMetaData.Category.IsEmpty())
		{
			Context.Function->SetMetaData(FBlueprintMetadata::MD_FunctionCategory, *FunctionMetaData.Category.ToString());
		}

		// Set up the function keywords
		if (!FunctionMetaData.Keywords.IsEmpty())
		{
			Context.Function->SetMetaData(FBlueprintMetadata::MD_FunctionKeywords, *FunctionMetaData.Keywords.ToString());
		}

		// Set up the function compact node title
		if (!FunctionMetaData.CompactNodeTitle.IsEmpty())
		{
			Context.Function->SetMetaData(FBlueprintMetadata::MD_CompactNodeTitle, *FunctionMetaData.CompactNodeTitle.ToString());
		}

		// Set up the function tooltip
		if (!FunctionMetaData.ToolTip.IsEmpty())
		{
			Context.Function->SetMetaData(FBlueprintMetadata::MD_Tooltip, *FunctionMetaData.ToolTip.ToString());
		}

		// Set as call in editor function
		if (FunctionMetaData.bCallInEditor)
		{
			Context.Function->SetMetaData(FBlueprintMetadata::MD_CallInEditor, TEXT( "true" ));
		}

		if (FunctionMetaData.bIsUnsafeDuringActorConstruction)
		{
			Context.Function->SetMetaData(FBlueprintMetadata::MD_UnsafeForConstructionScripts, TEXT("true"));
		}

		// Set appropriate metadata if the function is deprecated
		if (FunctionMetaData.bIsDeprecated)
		{
			Context.Function->SetMetaData(FBlueprintMetadata::MD_DeprecatedFunction, TEXT("true"));
			if (!FunctionMetaData.DeprecationMessage.IsEmpty())
			{
				Context.Function->SetMetaData(FBlueprintMetadata::MD_DeprecationMessage, *FunctionMetaData.DeprecationMessage);
			}
		}

		// Set the required function flags
		if (Context.CanBeCalledByKismet())
		{
			Context.Function->FunctionFlags |= FUNC_BlueprintCallable;
		}

		if (Context.IsInterfaceStub())
		{
			Context.Function->FunctionFlags |= FUNC_BlueprintEvent;
		}

		// Inherit extra flags from the entry node
		if (Context.EntryPoint)
		{
			Context.Function->FunctionFlags |= (EFunctionFlags)Context.EntryPoint->GetExtraFlags();
			
			if (UEdGraphPin* WorldContextPin = Context.EntryPoint->GetAutoWorldContextPin())
			{
				Context.Function->SetMetaData(FBlueprintMetadata::MD_WorldContext, *WorldContextPin->PinName.ToString());
			}
		}

		// First try to get the overriden function from the super class
		UFunction* OverridenFunction = Context.Function->GetSuperFunction();
		// If we couldn't find it, see if we can find an interface class in our inheritance to get it from
		if (!OverridenFunction && Context.Blueprint)
		{
			bool bInvalidInterface = false;
			OverridenFunction = FBlueprintEditorUtils::FindFunctionInImplementedInterfaces( Context.Blueprint, Context.Function->GetFName(), &bInvalidInterface );
			if(bInvalidInterface)
			{
				MessageLog.Warning(TEXT("Blueprint tried to implement invalid interface."));
			}
		}

		// Inherit flags and validate against overridden function if it exists
		if (OverridenFunction)
		{
			Context.Function->FunctionFlags |= (OverridenFunction->FunctionFlags & (FUNC_FuncInherit | FUNC_Public | FUNC_Protected | FUNC_Private | FUNC_BlueprintPure));

			if ((Context.Function->FunctionFlags & FUNC_AccessSpecifiers) != (OverridenFunction->FunctionFlags & FUNC_AccessSpecifiers))
			{
				MessageLog.Error(*LOCTEXT("IncompatibleAccessSpecifier_Error", "Access specifier is not compatible the parent function @@").ToString(), Context.EntryPoint);
			}

			const uint32 OverrideFlagsToCheck = (FUNC_FuncOverrideMatch & ~FUNC_AccessSpecifiers);
			if ((Context.Function->FunctionFlags & OverrideFlagsToCheck) != (OverridenFunction->FunctionFlags & OverrideFlagsToCheck))
			{
				MessageLog.Error(*LOCTEXT("IncompatibleOverrideFlags_Error", "Overriden function is not compatible with the parent function @@. Check flags: Exec, Final, Static.").ToString(), Context.EntryPoint);
			}

			// Copy metadata from parent function as well
			FMetaData::CopyMetadata(OverridenFunction, Context.Function);
		}
		else
		{
			// If this is the root of a blueprint-defined function or event, and if it's public, make it overrideable
			if( !Context.IsEventGraph() && !Context.Function->HasAnyFunctionFlags(FUNC_Private) )
			{
				Context.Function->FunctionFlags |= FUNC_BlueprintEvent;
			}
		}
		
		// Link it
		//@TODO: should this be in regular or reverse order?
		Context.Function->Next = Context.NewClass->Children;
		Context.NewClass->Children = Context.Function;

		// Add the function to it's owner class function name -> function map
		Context.NewClass->AddFunctionToFunctionMap(Context.Function, Context.Function->GetFName());
		if (UsePersistentUberGraphFrame() && Context.bIsUbergraph)
		{
			ensure(!NewClass->UberGraphFunction);
			NewClass->UberGraphFunction = Context.Function;
			NewClass->UberGraphFunction->FunctionFlags |= FUNC_UbergraphFunction;
			NewClass->UberGraphFunction->FunctionFlags |= FUNC_Final;
		}

		// Register nets from function entry/exit nodes first, even for skeleton compiles (as they form the signature)
		// We're violating the FNodeHandlingFunctor abstraction here because we want to make sure that the signature
		// matches even if all result nodes were pruned:
		bool bReturnNodeFound = false;
		for (UEdGraphNode* Node : Context.SourceGraph->Nodes)
		{
			if(Node->IsA(UK2Node_FunctionResult::StaticClass()))
			{
				bReturnNodeFound = true;
			}

			if (FNodeHandlingFunctor* Handler = NodeHandlers.FindRef(Node->GetClass()))
			{
				if (Handler->RequiresRegisterNetsBeforeScheduling())
				{
					Handler->RegisterNets(Context, Node);
				}
			}
		}

		if(!bReturnNodeFound &&
			!Context.IsEventGraph() &&
			!Context.bIsSimpleStubGraphWithNoParams &&
			Context.CanBeCalledByKismet() &&
			Context.Function->GetFName() != UEdGraphSchema_K2::FN_UserConstructionScript)
		{
			// dig into the (actual) source graph and find the original return node:
			UObject* Object = Context.MessageLog.FindSourceObject(Context.SourceGraph);
			if(Object)
			{
				UEdGraph* RealSourceGraph = Cast<UEdGraph>(Object);
				if( RealSourceGraph )
				{
					TArray<UK2Node_FunctionResult*> ResultNodes;
					RealSourceGraph->GetNodesOfClass<UK2Node_FunctionResult>(ResultNodes);
					if(ResultNodes.Num() > 0)
					{
						// Use whatever signature the first result node specifies:
						UK2Node_FunctionResult* FirstResultNode = ResultNodes[0];
						if (FNodeHandlingFunctor* Handler = NodeHandlers.FindRef(UK2Node_FunctionResult::StaticClass()))
						{
							if (Handler->RequiresRegisterNetsBeforeScheduling())
							{
								Handler->RegisterNets(Context, FirstResultNode);
							}
						}

						// We can't reliably warn here because FBlueprintGraphActionDetails::OnAddNewOutputClicked calls 
						// OnParamsChanged immediately after adding a param to a single node, so only the first result node
						// is guaranteed to be coherent/up to date.. For now we just rely on the editor to make uniform 
						// result nodes.
					}
				}
			}
		}

		FField::FLinkedListBuilder FunctionPropertyListBuilder(&(Context.Function->ChildProperties));

		// Create input/output parameter variables, this must occur before registering nets so that the properties are in place
		CreateParametersForFunction(Context, ParentFunction ? ParentFunction : OverridenFunction, FunctionPropertyListBuilder);

		if(bImmediatelyGenerateLocals)
		{
			CreateLocalsAndRegisterNets(Context, FunctionPropertyListBuilder);
		}
		else
		{
			// Fix up the return value - this used to be done by CreateLocalVariablesForFunction.
			// This should probably be done in CreateParametersForFunction
			for (TFieldIterator<FProperty> It(Context.Function); It && (It->PropertyFlags & CPF_Parm); ++It)
			{
				FProperty* Property = *It;
				if ((Property->GetFName() == UEdGraphSchema_K2::PN_ReturnValue) && Property->HasAnyPropertyFlags(CPF_OutParm))
				{
					Property->SetPropertyFlags(CPF_ReturnParm);
				}
			}
		}

		// Validate AccessSpecifier
		const uint32 AccessSpecifierFlag = FUNC_AccessSpecifiers & Context.EntryPoint->GetExtraFlags();
		const bool bAcceptedAccessSpecifier = 
			(0 == AccessSpecifierFlag) || (FUNC_Public == AccessSpecifierFlag) || (FUNC_Protected == AccessSpecifierFlag) || (FUNC_Private == AccessSpecifierFlag);
		if (!bAcceptedAccessSpecifier)
		{
			MessageLog.Warning(*LOCTEXT("WrongAccessSpecifier_Error", "Wrong access specifier @@").ToString(), Context.EntryPoint);
		}

		Context.Function->FunctionFlags |= (EFunctionFlags)Context.GetNetFlags();
		
		// Parameter list needs to be linked before signatures are compared. 
		Context.Function->StaticLink(true);

		// Make sure the function signature is valid if this is an override
		if (ParentFunction != nullptr)
		{
			// Verify the signature
			if (!ParentFunction->IsSignatureCompatibleWith(Context.Function))
			{
				FString SignatureClassName("");
				if (const UClass* SignatureClass = Context.EntryPoint->FunctionReference.GetMemberParentClass())
				{
					SignatureClassName = SignatureClass->GetName();
				}
				MessageLog.Error(
					*FText::Format(
						LOCTEXT("OverrideFunctionDifferentSignature_ErrorFmt", "Cannot override '{0}::{1}' at @@ which was declared in a parent with a different signature"),
						FText::FromString(SignatureClassName),
						FText::FromString(NewFunctionNameString)
					).ToString(),
					Context.EntryPoint
				);
			}
			const bool bEmptyCase = (0 == AccessSpecifierFlag);
			const bool bDifferentAccessSpecifiers = AccessSpecifierFlag != (ParentFunction->FunctionFlags & FUNC_AccessSpecifiers);
			if (!bEmptyCase && bDifferentAccessSpecifiers)
			{
				MessageLog.Warning(*LOCTEXT("IncompatibleAccessSpecifier_Error", "Access specifier is not compatible the parent function @@").ToString(), Context.EntryPoint);
			}

			EFunctionFlags const ParentNetFlags = (ParentFunction->FunctionFlags & FUNC_NetFuncFlags);
			if (ParentNetFlags != Context.GetNetFlags())
			{
				MessageLog.Error(*LOCTEXT("MismatchedNetFlags_Error", "@@ function's net flags don't match parent function's flags").ToString(), Context.EntryPoint);
				
				// clear the existing net flags
				Context.Function->FunctionFlags &= ~(FUNC_NetFuncFlags);
				// have to replace with the parent's net flags, or this will   
				// trigger an assert in Link()
				Context.Function->FunctionFlags |= ParentNetFlags;
			}
		}

		////////////////////////////////////////

		if (Context.IsDelegateSignature())
		{
			Context.Function->FunctionFlags |= FUNC_Delegate;

			// We really don't want to find our parent's delegate property and accidentally 
			// overwrite the signature function, so EFieldIterationFlags::IncludeSuper is excluded:
			const EFieldIterationFlags IterationFlags = EFieldIterationFlags::Default & ~EFieldIterationFlags::IncludeSuper;
			if (FMulticastDelegateProperty* Property = FindFProperty<FMulticastDelegateProperty>(
					NewClass, Context.DelegateSignatureName, IterationFlags))
			{
				Property->SignatureFunction = Context.Function;
			}
			else
			{
				MessageLog.Warning(*LOCTEXT("NoDelegateProperty_Error", "No delegate property found for @@").ToString(), Context.SourceGraph);
			}
		}

		if (Context.EntryPoint)
		{
			if (Context.EntryPoint->MetaData.HasMetaData(FBlueprintMetadata::MD_FieldNotify) && NewClass->ImplementsInterface(UNotifyFieldValueChanged::StaticClass()))
			{
				ensure(!NewClass->FieldNotifies.Contains(FFieldNotificationId(Context.Function->GetFName())));
				NewClass->FieldNotifies.Add(FFieldNotificationId(Context.Function->GetFName()));
			}
		}
	}
	else
	{
		MessageLog.Error(*LOCTEXT("NoRootNodeFound_Error", "Could not find a root node for the graph @@").ToString(), Context.SourceGraph);
	}
}

void FKismetCompilerContext::InitializeGeneratedEventNodes(EInternalCompilerFlags InternalFlags)
{
	if (GeneratedMulticastDelegateProps.IsEmpty())
	{
		return;
	}

	TArray<UK2Node_GeneratedBoundEvent*> EventNodes;
	ConsolidatedEventGraph->GetNodesOfClass<UK2Node_GeneratedBoundEvent>(EventNodes);

	for (FMulticastDelegateProperty* GeneratedMulticastDelegateProp : GeneratedMulticastDelegateProps)
	{
		if (UClass* OwningClass = GeneratedMulticastDelegateProp->GetOwner<UClass>())
		{
			// Don't associate the skeleton BPGC with the precompiled function (Skeleton is not authoritative).
			if (OwningClass->GetAuthoritativeClass() != OwningClass)
			{
				continue;
			}
		}

		auto FindNodeWithDelegateName = [GeneratedMulticastDelegateProp](UK2Node_GeneratedBoundEvent* EventNode)
		{
			if (EventNode && GeneratedMulticastDelegateProp)
			{
				if (EventNode->EventReference.GetMemberName() == GeneratedMulticastDelegateProp->GetFName())
				{
					return true;
				}
			}

			return false;
		};
		
		if (UK2Node_GeneratedBoundEvent** EventNodeItr = EventNodes.FindByPredicate(FindNodeWithDelegateName))
		{
			UK2Node_GeneratedBoundEvent* EventNode = *EventNodeItr;

			auto FindFunctionWithDelegateName = [GeneratedMulticastDelegateProp, EventNode](const FKismetFunctionContext& FunctionContext)
			{
				if (FunctionContext.Function && FunctionContext.Function->GetFName() == EventNode->CustomFunctionName)
				{
					if (EventNode->EventReference.GetMemberName() == GeneratedMulticastDelegateProp->GetFName())
					{
						return true;
					}
				}

				return false;
			};

			if (const FKismetFunctionContext* PrecompiledFunctionItr = Algo::FindByPredicate(FunctionList, FindFunctionWithDelegateName))
			{
				const FKismetFunctionContext& PrecompiledFunction = *PrecompiledFunctionItr;
				GeneratedMulticastDelegateProp->SignatureFunction = PrecompiledFunction.Function;
				EventNode->InitializeGeneratedBoundEventParams(GeneratedMulticastDelegateProp);
			}
		}
	}
}

void FKismetCompilerContext::PreCompileUpdateBlueprintOnLoad(UBlueprint* BP)
{
	check(BP);
	if (BP->GetLinkerCustomVersion(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::BlueprintPinsUseRealNumbers)
	{
		UE::KismetCompiler::Private::MatchNodesToDelegateSignatures(BP);
	}
}

/** Inserts a new item into an array in a sorted position; using an externally stored sort index map */
template<typename DataType, typename SortKeyType>
void OrderedInsertIntoArray(TArray<DataType>& Array, const TMap<DataType, SortKeyType>& SortKeyMap, const DataType& NewItem)
{
	const SortKeyType NewItemKey = SortKeyMap.FindChecked(NewItem);

	for (int32 i = 0; i < Array.Num(); ++i)
	{
		DataType& TestItem = Array[i];
		const SortKeyType TestItemKey = SortKeyMap.FindChecked(TestItem);

		if (TestItemKey > NewItemKey)
		{
			Array.Insert(NewItem, i);
			return;
		}
	}

	Array.Add(NewItem);
}

/**
 * Second phase of compiling a function graph
 *   - Generates executable code and performs final validation
 */
void FKismetCompilerContext::CompileFunction(FKismetFunctionContext& Context)
{
	BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_CompileFunction);
		
	check(Context.IsValid());

	// Generate statements for each node in the linear execution order (which should roughly correspond to the final execution order)
	TMap<UEdGraphNode*, int32> SortKeyMap;
	int32 NumNodesAtStart = Context.LinearExecutionList.Num();
	for (int32 i = 0; i < Context.LinearExecutionList.Num(); ++i)
	{
		UEdGraphNode* Node = Context.LinearExecutionList[i];
		SortKeyMap.Add(Node, i);

		const FString NodeComment = Node->NodeComment.IsEmpty() ? Node->GetName() : Node->NodeComment;
		const bool bPureNode = IsNodePure(Node);
		// Debug comments
		if (KismetCompilerDebugOptions::EmitNodeComments)
		{
			FBlueprintCompiledStatement& Statement = Context.AppendStatementForNode(Node);
			Statement.Type = KCST_Comment;
			Statement.Comment = NodeComment;
		}

		// Debug opcode insertion point
		if (Context.IsDebuggingOrInstrumentationRequired())
		{
			if (!bPureNode)
			{
				UEdGraphPin* ExecPin = nullptr;
				bool bEmitDebuggingSite = true;

				if (Context.IsEventGraph() && (Node->IsA(UK2Node_FunctionEntry::StaticClass())))
				{
					// The entry point in the ubergraph is a non-visual construct, and will lead to some
					// other 'fake' entry point such as an event or latent action.  Therefore, don't create
					// debug data for the behind-the-scenes entry point, only for the user-visible ones.
					bEmitDebuggingSite = false;
				}

				if (bEmitDebuggingSite)
				{
					FBlueprintCompiledStatement& Statement = Context.AppendStatementForNode(Node);
					Statement.Type = Context.GetBreakpointType();
					Statement.ExecContext = ExecPin;
					Statement.Comment = NodeComment;
				}
			}
		}

		// Let the node handlers try to compile it
		if (FNodeHandlingFunctor* Handler = NodeHandlers.FindRef(Node->GetClass()))
		{
			Handler->Compile(Context, Node);
		}
		else
		{
			MessageLog.Error(
				*FText::Format(
					LOCTEXT("UnexpectedNodeTypeWhenCompilingFunc_ErrorFmt", "Unexpected node type {0} encountered in execution chain at @@"),
					FText::FromString(Node->GetClass()->GetName())
				).ToString(),
				Node
			);
		}
	}
	
	if (Context.ImplicitCastMap.Num() > 0)
	{
		// Only issue a warning for pins that belong to nodes in the execution path.
		// We only compile those particular nodes, which is where implicit cast fixup occurs.

		TArray<UEdGraphPin*> UnhandledPins;
		for (const auto& It : Context.ImplicitCastMap)
		{
			UEdGraphNode* OwningNode = It.Key->GetOwningNode();
			if (SortKeyMap.Find(OwningNode))
			{
				UnhandledPins.Add(It.Key);
			}
			else
			{
				UE_LOGF(LogK2Compiler, Verbose, "Skipping implicit cast check for pin '%ls' in function '%ls'. Owning node '%ls' is not in the execution list.",
					*It.Key->PinName.ToString(),
					*Context.Function->GetName(),
					*It.Key->GetOwningNode()->GetName());
			}
		}

		if (UnhandledPins.Num() > 0)
		{
			UE_LOGF(LogK2Compiler, Warning, "Unhandled implicit casts found during compilation of function '%ls'!",
				*Context.Function->GetFullName());

			for (UEdGraphPin* Pin : UnhandledPins)
			{
				UE_LOGF(LogK2Compiler, Warning, "\tPin '%ls' was not handled by node '%ls'!",
					*Pin->PinName.ToString(),
					*Pin->GetOwningNode()->GetName());
			}
		}
	}

	// The LinearExecutionList should be immutable at this point
	check(Context.LinearExecutionList.Num() == NumNodesAtStart);

	// Now pull out pure chains and inline their generated code into the nodes that need it
	TMap< UEdGraphNode*, TSet<UEdGraphNode*> > PureNodesNeeded;
	
	for (int32 TestIndex = 0; TestIndex < Context.LinearExecutionList.Num(); )
	{
		UEdGraphNode* Node = Context.LinearExecutionList[TestIndex];

		// List of pure nodes this node depends on.
		bool bHasAntecedentPureNodes = PureNodesNeeded.Contains(Node);

		if (IsNodePure(Node))
		{
			// For profiling purposes, find the statement that marks the function's entry point.
			FBlueprintCompiledStatement* ProfilerStatement = nullptr;
			TArray<FBlueprintCompiledStatement*>* SourceStatementList = Context.StatementsPerNode.Find(Node);
			const bool bDidNodeGenerateCode = SourceStatementList != nullptr && SourceStatementList->Num() > 0;
			if (bDidNodeGenerateCode)
			{
				for (FBlueprintCompiledStatement* Statement : *SourceStatementList)
				{
					if (Statement && Statement->Type == KCST_InstrumentedPureNodeEntry)
					{
						ProfilerStatement = Statement;
						break;
					}
				}
			}

			// Push this node to the requirements list of any other nodes using it's outputs, if this node had any real impact
			if (bDidNodeGenerateCode || bHasAntecedentPureNodes)
			{
				for (int32 PinIndex = 0; PinIndex < Node->Pins.Num(); ++PinIndex)
				{
					UEdGraphPin* Pin = Node->Pins[PinIndex];
					if (Pin->Direction == EGPD_Output && Pin->LinkedTo.Num() > 0)
					{
						// Record the pure node output pin, since it's linked
						if (ProfilerStatement)
						{
							ProfilerStatement->PureOutputContextArray.AddUnique(Pin);
						}

						for (UEdGraphPin* LinkedTo : Pin->LinkedTo)
						{
							UEdGraphNode* NodeUsingOutput = LinkedTo->GetOwningNode();
							if (NodeUsingOutput != nullptr)
							{
								// Add this node, as well as other nodes this node depends on
								TSet<UEdGraphNode*>& TargetNodesRequired = PureNodesNeeded.FindOrAdd(NodeUsingOutput);
								TargetNodesRequired.Add(Node);
								if (bHasAntecedentPureNodes)
								{
									TargetNodesRequired.Append(PureNodesNeeded.FindChecked(Node));
								}
							}
						}
					}
				}
			}

			// Remove it from the linear execution list; the dependent nodes will inline the code when necessary
			Context.LinearExecutionList.RemoveAt(TestIndex);
		}
		else
		{
			if (bHasAntecedentPureNodes)
			{
				// This node requires the output of one or more pure nodes, so that pure code needs to execute at this node

				// Sort the nodes by execution order index
				TSet<UEdGraphNode*>& AntecedentPureNodes = PureNodesNeeded.FindChecked(Node);
				TArray<UEdGraphNode*> SortedPureNodes;
				for (TSet<UEdGraphNode*>::TIterator It(AntecedentPureNodes); It; ++It)
				{
					OrderedInsertIntoArray(SortedPureNodes, SortKeyMap, *It);
				}

				// Inline their code
				for (int32 i = 0; i < SortedPureNodes.Num(); ++i)
				{
					UEdGraphNode* NodeToInline = SortedPureNodes[SortedPureNodes.Num() - 1 - i];

					Context.CopyAndPrependStatements(Node, NodeToInline);
				}
			}

			// Proceed to the next node
			++TestIndex;
		}
	}

	// Propagate thread-safe flags in this first pass. Also gets called from SetCalculatedMetaDataAndFlags in the second
	// pass to catch skeleton class generation
	if (Context.EntryPoint->MetaData.bThreadSafe)
	{
		Context.Function->SetMetaData(FBlueprintMetadata::MD_ThreadSafe, TEXT("true"));
	}
}

/**
 * Final phase of compiling a function graph; called after all functions have had CompileFunction called
 *   - Patches up cross-references, etc..., and performs final validation
 */
void FKismetCompilerContext::PostcompileFunction(FKismetFunctionContext& Context)
{
	BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_PostcompileFunction);

	// The function links gotos, sorts statments, and merges adjacent ones. 
	Context.ResolveStatements();

	//@TODO: Code generation (should probably call backend here, not later)

	// Seal the function, it's done!
	FinishCompilingFunction(Context);
}

#if VALIDATE_UBER_GRAPH_PERSISTENT_FRAME
extern ENGINE_API int32 IncrementUberGraphSerialNumber();
#endif//VALIDATE_UBER_GRAPH_PERSISTENT_FRAME

/**
 * Handles final post-compilation setup, flags, creates cached values that would normally be set during deserialization, etc...
 */
void FKismetCompilerContext::FinishCompilingFunction(FKismetFunctionContext& Context)
{
	SetCalculatedMetaDataAndFlags( Context.Function, Context.EntryPoint, Schema );
	
#if VALIDATE_UBER_GRAPH_PERSISTENT_FRAME
	if( NewClass->UberGraphFunction == Context.Function )
	{
		NewClass->UberGraphFunctionKey = IncrementUberGraphSerialNumber();

		// if the old class uber graph function matches, just reuse that ID, this check means
		// that if child types aren't reinstanced we can still validate their uber graph:
		if(NewClass->UberGraphFunction && OldClass)
		{
			bool bSameLayout = FStructUtils::TheSameLayout(OldClass->UberGraphFunction, NewClass->UberGraphFunction);
			if(bSameLayout)
			{
				NewClass->UberGraphFunctionKey = OldClass->UberGraphFunctionKey;
			}
		}
	}
#endif//VALIDATE_UBER_GRAPH_PERSISTENT_FRAME

	// Check thread safety
	if (FBlueprintEditorUtils::HasFunctionBlueprintThreadSafeMetaData(Context.Function))
	{
		FKismetCompilerUtilities::CheckFunctionThreadSafety(Context, MessageLog);
	}

	// This is necessary to stop async loading and partial reloads from corrupting the compiled data
	// without marking the owning class for recompilation
	if (Context.Function)
	{
		Context.Function->SetFlags(RF_LoadCompleted);
	}
}

void FKismetCompilerContext::SetCalculatedMetaDataAndFlags(UFunction* Function, UK2Node_FunctionEntry* EntryNode, const UEdGraphSchema_K2* K2Schema)
{
	if(!ensure(Function) || !ensure(EntryNode))
	{
		return;
	}

	Function->Bind();
	Function->StaticLink(true);

	// Set function flags and calculate cached values so the class can be used immediately
	Function->ParmsSize = 0;
	Function->NumParms = 0;
	Function->ReturnValueOffset = MAX_uint16;
	Function->FirstPropertyToInit = nullptr;
	UEProperty_Private::FPropertyListBuilderPostConstructLink ConstructLink(&Function->FirstPropertyToInit);

	for (TFieldIterator<FProperty> PropIt(Function, EFieldIteratorFlags::ExcludeSuper); PropIt; ++PropIt)
	{
		FProperty* Property = *PropIt;
		if (Property->HasAnyPropertyFlags(CPF_Parm))
		{
			++Function->NumParms;
			Function->ParmsSize = IntCastChecked<int16, int32>(Property->GetOffset_ForUFunction() + Property->GetSize());

			if (Property->HasAnyPropertyFlags(CPF_OutParm))
			{
				Function->FunctionFlags |= FUNC_HasOutParms;
			}

			if (Property->HasAnyPropertyFlags(CPF_ReturnParm))
			{
				Function->ReturnValueOffset = IntCastChecked<uint16, int32>(Property->GetOffset_ForUFunction());
			}
		}
		else
		{
			if (!Property->HasAnyPropertyFlags(CPF_ZeroConstructor))
			{
				ConstructLink.AppendTerminated(*Property);
				Function->FunctionFlags |= FUNC_HasDefaults;
			}
		}
	}

	FKismetUserDeclaredFunctionMetadata& FunctionMetaData = EntryNode->MetaData;
	if (!FunctionMetaData.Category.IsEmpty())
	{
		Function->SetMetaData(FBlueprintMetadata::MD_FunctionCategory, *FunctionMetaData.Category.ToString());
	}

	// Set up the function keywords
	if (!FunctionMetaData.Keywords.IsEmpty())
	{
		Function->SetMetaData(FBlueprintMetadata::MD_FunctionKeywords, *FunctionMetaData.Keywords.ToString());
	}

	// Set up the function compact node title
	if (!FunctionMetaData.CompactNodeTitle.IsEmpty())
	{
		Function->SetMetaData(FBlueprintMetadata::MD_CompactNodeTitle, *FunctionMetaData.CompactNodeTitle.ToString());
	}

	// Add in any extra user-defined metadata, like tooltip
	if (!EntryNode->MetaData.ToolTip.IsEmpty())
	{
		Function->SetMetaData(FBlueprintMetadata::MD_Tooltip, *EntryNode->MetaData.ToolTip.ToString());
	}

	if (EntryNode->MetaData.bCallInEditor)
	{
		Function->SetMetaData(FBlueprintMetadata::MD_CallInEditor, TEXT("true"));
	}

	if (EntryNode->MetaData.bIsDeprecated)
	{
		Function->SetMetaData(FBlueprintMetadata::MD_DeprecatedFunction, TEXT("true"));
		
		if (!EntryNode->MetaData.DeprecationMessage.IsEmpty())
		{
			Function->SetMetaData(FBlueprintMetadata::MD_DeprecationMessage, *(EntryNode->MetaData.DeprecationMessage));
		}
	}

	// Just copy the thread-safe flag with no checking, we verify thread safety elsewhere
	if (EntryNode->MetaData.bThreadSafe)
	{
		Function->SetMetaData(FBlueprintMetadata::MD_ThreadSafe, TEXT("true"));
	}

	if (EntryNode->MetaData.bIsUnsafeDuringActorConstruction)
	{
		Function->SetMetaData(FBlueprintMetadata::MD_UnsafeForConstructionScripts, TEXT("true"));
	}
	
	if (UEdGraphPin* WorldContextPin = EntryNode->GetAutoWorldContextPin())
	{
		Function->SetMetaData(FBlueprintMetadata::MD_WorldContext, *WorldContextPin->PinName.ToString());
	}

	SetDefaultInputValueMetaData(Function, EntryNode->UserDefinedPins);

	if(UFunction* OverriddenFunction = Function->GetSuperFunction())
	{
		// Copy metadata from parent function as well
		FMetaData::CopyMetadata(OverriddenFunction, Function);

		// Native thread safety metadata works on the class level as well as functions, so we need to check that too.
		if(FBlueprintEditorUtils::HasFunctionBlueprintThreadSafeMetaData(OverriddenFunction))
		{
			Function->SetMetaData(FBlueprintMetadata::MD_ThreadSafe, TEXT("true"));
		}
	}

	for (const TPair<FName, FString>& Entry : FunctionMetaData.GetMetaDataMap())
	{
		Function->SetMetaData(Entry.Key, *Entry.Value);
	}
}

void FKismetCompilerContext::SetDefaultInputValueMetaData(UFunction* Function, const TArray< TSharedPtr<FUserPinInfo> >& InputData)
{
	for (const TSharedPtr<FUserPinInfo>& InputDataPtr : InputData)
	{
		if ( InputDataPtr.IsValid() &&
			!InputDataPtr->PinName.IsNone() &&
			(InputDataPtr->PinName != UEdGraphSchema_K2::PN_Self) &&
			(InputDataPtr->PinType.PinCategory != UEdGraphSchema_K2::PC_Exec) && 
			(InputDataPtr->PinType.PinCategory != UEdGraphSchema_K2::PC_Object) &&
			(InputDataPtr->PinType.PinCategory != UEdGraphSchema_K2::PC_Class) &&
			(InputDataPtr->PinType.PinCategory != UEdGraphSchema_K2::PC_Interface) )
		{
			Function->SetMetaData(InputDataPtr->PinName, *InputDataPtr->PinDefaultValue);
		}
	}
}

/**
 * Handles adding the implemented interface information to the class
 */
void FKismetCompilerContext::AddInterfacesFromBlueprint(UClass* Class)
{
	// Make sure we actually have some interfaces to implement
	if( Blueprint->ImplementedInterfaces.Num() == 0 )
	{
		return;
	}

	// Iterate over all implemented interfaces, and add them to the class
	for(int32 i = 0; i < Blueprint->ImplementedInterfaces.Num(); i++)
	{
		UClass* Interface = Blueprint->ImplementedInterfaces[i].Interface;
		if( Interface )
		{
			// Make sure it's a valid interface
			check(Interface->HasAnyClassFlags(CLASS_Interface));

			//propogate the inheritable ClassFlags
			Class->ClassFlags |= (Interface->ClassFlags) & CLASS_ScriptInherit;

			new (Class->Interfaces) FImplementedInterface(Interface, 0, true);
		}
	}
}

/**
 * Handles final post-compilation setup, flags, creates cached values that would normally be set during deserialization, etc...
 */
void FKismetCompilerContext::FinishCompilingClass(UClass* Class)
{
	UClass* ParentClass = Class->GetSuperClass();

	FBlueprintEditorUtils::RecreateClassMetaData(Blueprint, Class, false);

	if (ParentClass != NULL)
	{
		// Propagate the new parent's inheritable class flags
		Class->ReferenceSchema.Reset();
		FBlueprintClassFlagUtils::PropagateParentClassFlags_FinishCompilingClass(Class);

		// If the Blueprint was marked as deprecated, then flag the class as deprecated.
		if(Blueprint->bDeprecate)
		{
			Class->ClassFlags |= CLASS_Deprecated;
		}

		// If the flag is inherited, this will keep the bool up-to-date
		Blueprint->bDeprecate = (Class->ClassFlags & CLASS_Deprecated) == CLASS_Deprecated;
		
		// If the Blueprint was marked as abstract, then flag the class as abstract.
		if (Blueprint->bGenerateAbstractClass)
		{
			NewClass->ClassFlags |= CLASS_Abstract;
		}
		Blueprint->bGenerateAbstractClass = (Class->ClassFlags & CLASS_Abstract) == CLASS_Abstract;	

		// Add the description to the tooltip
		static const FName NAME_Tooltip(TEXT("Tooltip"));
		if (!Blueprint->BlueprintDescription.IsEmpty())
		{
			Class->SetMetaData(NAME_Tooltip, *Blueprint->BlueprintDescription);
		}
		else
		{
			Class->RemoveMetaData(NAME_Tooltip);
		}

		static const FName NAME_DisplayName(TEXT("DisplayName"));
		if (!Blueprint->BlueprintDisplayName.IsEmpty())
		{
			Class->SetMetaData(FBlueprintMetadata::MD_DisplayName, *Blueprint->BlueprintDisplayName);
		}
		else
		{
			Class->RemoveMetaData(NAME_DisplayName);
		}

		// Add the namespace (if set) to class metadata
		if (!Blueprint->BlueprintNamespace.IsEmpty())
		{
			Class->SetMetaData(FBlueprintMetadata::MD_Namespace, *Blueprint->BlueprintNamespace);
		}
		else
		{
			Class->RemoveMetaData(FBlueprintMetadata::MD_Namespace);
		}

		// Copy the category info from the parent class
#if WITH_EDITORONLY_DATA
		
		// Blueprinted Components are always Blueprint Spawnable
		if (ParentClass->IsChildOf(UActorComponent::StaticClass()))
		{
			FComponentTypeRegistry::Get().InvalidateClass(Class);
		}
#endif

		// Add in additional flags implied by the blueprint
		switch (Blueprint->BlueprintType)
		{
			case BPTYPE_MacroLibrary:
				Class->ClassFlags |= CLASS_Abstract | CLASS_NotPlaceable;
				break;
			case BPTYPE_Const:
				Class->ClassFlags |= CLASS_Const;
				break;
		}

		//@TODO: Might want to be able to specify some of these here too
	}

	// Add in any other needed flags
	Class->ClassFlags |= (CLASS_CompiledFromBlueprint);
	Class->ClassFlags &= ~CLASS_ReplicationDataIsSetUp;

	FBlueprintClassFlagUtils::AppendPropertyBasedClassFlags(Class);

	// Verify class metadata as needed
	if (FBlueprintEditorUtils::IsInterfaceBlueprint(Blueprint))
	{
		ensure( NewClass->HasAllClassFlags( CLASS_Interface ) );
	}

	{
		UBlueprintGeneratedClass* BPGClass = Cast<UBlueprintGeneratedClass>(Class);
		check(BPGClass);

		BPGClass->ComponentTemplates = Blueprint->ComponentTemplates;
		BPGClass->Timelines = Blueprint->Timelines;
		BPGClass->SimpleConstructionScript = Blueprint->SimpleConstructionScript;
		BPGClass->InheritableComponentHandler = Blueprint->InheritableComponentHandler;
		BPGClass->ComponentClassOverrides = Blueprint->ComponentClassOverrides;
	}

	//@TODO: Not sure if doing this again is actually necessary
	// It will be if locals get promoted to class scope during function compilation, but that should ideally happen during Precompile or similar
	Class->Bind();
	
	// Ensure that function netflags equate to any super function in a parent BP prior to linking; it may have been changed by the user
	// and won't be reflected in the child class until it is recompiled. Without this, UClass::Link() will assert if they are out of sync.
	for (UField* Field = Class->Children; Field; Field = Field->Next)
	{
		UFunction* Function = dynamic_cast<UFunction*>(Field);
		if (Function != nullptr)
		{
			UFunction* ParentFunction = Function->GetSuperFunction();
			if (ParentFunction != nullptr)
			{
				const EFunctionFlags ParentNetFlags = (ParentFunction->FunctionFlags & FUNC_NetFuncFlags);
				if (ParentNetFlags != (Function->FunctionFlags & FUNC_NetFuncFlags))
				{
					Function->FunctionFlags &= ~FUNC_NetFuncFlags;
					Function->FunctionFlags |= ParentNetFlags;
				}
			}
		}
	}

	Class->StaticLink(true);
	Class->SetUpRuntimeReplicationData();

	// Create the default object for this class
	FKismetCompilerUtilities::CompileDefaultProperties(Class);

	CastChecked<UBlueprintGeneratedClass>(Class)->InitializeFieldNotifies();

	AActor* ActorCDO = Cast<AActor>(Class->GetDefaultObject());
	if (ActorCDO)
	{
		ensureMsgf(!ActorCDO->bExchangedRoles, TEXT("Your CDO has had ExchangeNetRoles called on it (likely via RerunConstructionScripts) which should never have happened. This will cause issues replicating this actor over the network due to mutated transient data!"));
	}
}

void FKismetCompilerContext::BuildDynamicBindingObjects(UBlueprintGeneratedClass* Class)
{
	Class->DynamicBindingObjects.Empty();

	for (FKismetFunctionContext& FunctionContext : FunctionList)
	{
		for (UEdGraphNode* GraphNode : FunctionContext.SourceGraph->Nodes)
		{
			UK2Node* Node = Cast<UK2Node>(GraphNode);

			if (Node)
			{
				UClass* DynamicBindingClass = Node->GetDynamicBindingClass();

				if (DynamicBindingClass)
				{
					UDynamicBlueprintBinding* DynamicBindingObject = UBlueprintGeneratedClass::GetDynamicBindingObject(Class, DynamicBindingClass);
					if (DynamicBindingObject == NULL)
					{
						DynamicBindingObject = NewObject<UDynamicBlueprintBinding>(Class, DynamicBindingClass);
						Class->DynamicBindingObjects.Add(DynamicBindingObject);
					}
					Node->RegisterDynamicBinding(DynamicBindingObject);
				}
			}
		}
	}
}

/**
 * Helper function to create event node for a given pin on a timeline node.
 * @param TimelineNode	The timeline node to create the node event for
 * @param SourceGraph	The source graph to create the event node in
 * @param FunctionName	The function to use as the custom function for the event node
 * @param PinName		The pin name to redirect output from, into the pin of the node event
 * @param ExecFuncName	The event signature name that the event node implements
 */ 
void FKismetCompilerContext::CreatePinEventNodeForTimelineFunction(UK2Node_Timeline* TimelineNode, UEdGraph* SourceGraph, FName FunctionName, const FName PinName, FName ExecFuncName)
{
	UK2Node_Event* TimelineEventNode = SpawnIntermediateNode<UK2Node_Event>(TimelineNode, SourceGraph);
	TimelineEventNode->EventReference.SetExternalMember(FunctionName, UTimelineComponent::StaticClass());
	TimelineEventNode->CustomFunctionName = FunctionName; // Make sure we name this function the thing we are expecting
	TimelineEventNode->bInternalEvent = true;
	TimelineEventNode->AllocateDefaultPins();

	// Move any links from 'update' pin to the 'update event' node
	UEdGraphPin* UpdatePin = TimelineNode ? TimelineNode->FindPin(PinName) : nullptr;
	ensureMsgf(UpdatePin, TEXT("Timeline '%s' has no pin '%s'"), *GetPathNameSafe(TimelineNode), *PinName.ToString());

	UEdGraphPin* UpdateOutput = Schema->FindExecutionPin(*TimelineEventNode, EGPD_Output);

	if(UpdatePin && UpdateOutput)
	{
		MovePinLinksToIntermediate(*UpdatePin, *UpdateOutput);
	}
}

UK2Node_CallFunction* FKismetCompilerContext::CreateCallTimelineFunction(UK2Node_Timeline* TimelineNode, UEdGraph* SourceGraph, FName FunctionName, UEdGraphPin* TimelineVarPin, UEdGraphPin* TimelineFunctionPin)
{
	// Create 'call play' node
	UK2Node_CallFunction* CallNode = SpawnIntermediateNode<UK2Node_CallFunction>(TimelineNode, SourceGraph);
	CallNode->FunctionReference.SetExternalMember(FunctionName, UTimelineComponent::StaticClass());
	CallNode->AllocateDefaultPins();

	// Wire 'get timeline' to 'self' pin of function call
	UEdGraphPin* CallSelfPin = CallNode->FindPinChecked(UEdGraphSchema_K2::PN_Self);
	TimelineVarPin->MakeLinkTo(CallSelfPin);

	// Move any exec links from 'play' pin to the 'call play' node
	UEdGraphPin* CallExecInput = Schema->FindExecutionPin(*CallNode, EGPD_Input);
	MovePinLinksToIntermediate(*TimelineFunctionPin, *CallExecInput);
	return CallNode;
}

/** Expand timeline nodes into necessary nodes */
void FKismetCompilerContext::ExpandTimelineNodes(UEdGraph* SourceGraph)
{
	// Timeline Pair helper
	struct FTimelinePair
	{
	public:
		FTimelinePair( UK2Node_Timeline* InNode, UTimelineTemplate* InTemplate )
		: Node( InNode )
		, Template( InTemplate )
		{
		}
	public:
		UK2Node_Timeline* const Node;
		UTimelineTemplate* Template;
	};

	TArray<FName> TimelinePlayNodes;
	TArray<FTimelinePair> Timelines;
	// Extract timeline pairings and external play nodes
	for (int32 ChildIndex = 0; ChildIndex < SourceGraph->Nodes.Num(); ++ChildIndex)
	{
		if (UK2Node_Timeline* TimelineNode = Cast<UK2Node_Timeline>( SourceGraph->Nodes[ChildIndex]))
		{
			UTimelineTemplate* Timeline = Blueprint->FindTimelineTemplateByVariableName(TimelineNode->TimelineName);
			if (Timeline != NULL)
			{
				Timelines.Add(FTimelinePair( TimelineNode, Timeline ));
			}
		}
		else if( UK2Node_VariableGet* VarNode = Cast<UK2Node_VariableGet>( SourceGraph->Nodes[ChildIndex] ))
		{
			// Check for Timeline Variable Get Nodes
			UEdGraphPin* const ValuePin = VarNode->GetValuePin();

			if( ValuePin && ValuePin->LinkedTo.Num() > 0  )
			{
				UClass* ValueClass = ValuePin->PinType.PinSubCategoryObject.IsValid() ? Cast<UClass>( ValuePin->PinType.PinSubCategoryObject.Get() ) : nullptr;
				if( ValueClass == UTimelineComponent::StaticClass() )
				{
					const FName PinName = ValuePin->PinName;
					if( UTimelineTemplate* TimelineTemplate = Blueprint->FindTimelineTemplateByVariableName( PinName ))
					{
						TimelinePlayNodes.Add( PinName );
					}
				}
			}
		}
	}
	// Expand and validate timelines
	for ( const FTimelinePair& TimelinePair : Timelines )
	{
		UK2Node_Timeline* const TimelineNode = TimelinePair.Node;
		UTimelineTemplate* Timeline = TimelinePair.Template;

		if (bIsFullCompile)
		{
			UEdGraphPin* PlayPin = TimelineNode->GetPlayPin();
			bool bPlayPinConnected = (PlayPin->LinkedTo.Num() > 0);

			UEdGraphPin* PlayFromStartPin = TimelineNode->GetPlayFromStartPin();
			bool bPlayFromStartPinConnected = (PlayFromStartPin->LinkedTo.Num() > 0);

			UEdGraphPin* StopPin = TimelineNode->GetStopPin();
			bool bStopPinConnected = (StopPin->LinkedTo.Num() > 0);

			UEdGraphPin* ReversePin = TimelineNode->GetReversePin();
			bool bReversePinConnected = (ReversePin->LinkedTo.Num() > 0);

			UEdGraphPin* ReverseFromEndPin = TimelineNode->GetReverseFromEndPin();
			bool bReverseFromEndPinConnected = (ReverseFromEndPin->LinkedTo.Num() > 0);

			UEdGraphPin* SetTimePin = TimelineNode->GetSetNewTimePin();
			bool bSetNewTimePinConnected = (SetTimePin->LinkedTo.Num() > 0);

			UEdGraphPin* UpdatePin = TimelineNode->GetUpdatePin();
			bool bUpdatePinConnected = (UpdatePin->LinkedTo.Num() > 0);

			UEdGraphPin* FinishedPin = TimelineNode->GetFinishedPin();
			bool bFinishedPinConnected = (FinishedPin->LinkedTo.Num() > 0);


			// Set the timeline template as wired/not wired for component pruning later
			const bool bWiredIn = bPlayPinConnected || bPlayFromStartPinConnected || bStopPinConnected || bReversePinConnected || bReverseFromEndPinConnected || bSetNewTimePinConnected;

			// Only create nodes for play/stop if they are actually connected - otherwise we get a 'unused node being pruned' warning
			if (bWiredIn)
			{
				// First create 'get var' node to get the timeline object
				UK2Node_VariableGet* GetTimelineNode = SpawnIntermediateNode<UK2Node_VariableGet>(TimelineNode, SourceGraph);
				GetTimelineNode->VariableReference.SetSelfMember(TimelineNode->TimelineName);
				GetTimelineNode->AllocateDefaultPins();

				// Debug data: Associate the timeline node instance with the property that was created earlier
				if (FProperty* AssociatedTimelineInstanceProperty = TimelineToMemberVariableMap.FindChecked(Timeline))
				{
					UObject* TrueSourceObject = MessageLog.FindSourceObject(TimelineNode);
					NewClass->GetDebugData().RegisterClassPropertyAssociation(TrueSourceObject, AssociatedTimelineInstanceProperty);
				}

				// Get the variable output pin
				UEdGraphPin* TimelineVarPin = GetTimelineNode->FindPin(TimelineNode->TimelineName);

				// This might fail if this is the first compile after adding the timeline (property doesn't exist yet) - in that case, manually add the output pin
				if (TimelineVarPin == nullptr)
				{
					TimelineVarPin = GetTimelineNode->CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Object, UTimelineComponent::StaticClass(), TimelineNode->TimelineName);
				}

				if (bPlayPinConnected)
				{
					static FName PlayName(GET_FUNCTION_NAME_CHECKED(UTimelineComponent, Play));
					CreateCallTimelineFunction(TimelineNode, SourceGraph, PlayName, TimelineVarPin, PlayPin);
				}

				if (bPlayFromStartPinConnected)
				{
					static FName PlayFromStartName(GET_FUNCTION_NAME_CHECKED(UTimelineComponent, PlayFromStart));
					CreateCallTimelineFunction(TimelineNode, SourceGraph, PlayFromStartName, TimelineVarPin, PlayFromStartPin);
				}

				if (bStopPinConnected)
				{
					static FName StopName(GET_FUNCTION_NAME_CHECKED(UTimelineComponent, Stop));
					CreateCallTimelineFunction(TimelineNode, SourceGraph, StopName, TimelineVarPin, StopPin);
				}

				if (bReversePinConnected)
				{
					static FName ReverseName(GET_FUNCTION_NAME_CHECKED(UTimelineComponent, Reverse));
					CreateCallTimelineFunction(TimelineNode, SourceGraph, ReverseName, TimelineVarPin, ReversePin);
				}

				if (bReverseFromEndPinConnected)
				{
					static FName ReverseFromEndName(GET_FUNCTION_NAME_CHECKED(UTimelineComponent, ReverseFromEnd));
					CreateCallTimelineFunction(TimelineNode, SourceGraph, ReverseFromEndName, TimelineVarPin, ReverseFromEndPin);
				}

				if (bSetNewTimePinConnected)
				{
					UEdGraphPin* NewTimePin = TimelineNode->GetNewTimePin();

					static FName SetNewTimeName(GET_FUNCTION_NAME_CHECKED(UTimelineComponent, SetNewTime));
					UK2Node_CallFunction* CallNode = CreateCallTimelineFunction(TimelineNode, SourceGraph, SetNewTimeName, TimelineVarPin, SetTimePin);

					if (CallNode && NewTimePin)
					{
						UEdGraphPin* InputPin = CallNode->FindPinChecked(TEXT("NewTime"));
						MovePinLinksToIntermediate(*NewTimePin, *InputPin);
					}
				}
			}
		}

		// Create event to call on each update
		UFunction* EventSigFunc = UTimelineComponent::GetTimelineEventSignature();

		// Create event nodes for any event tracks
		for (const FTTEventTrack& EventTrack : Timeline->EventTracks)
		{
			CreatePinEventNodeForTimelineFunction(TimelineNode, SourceGraph, EventTrack.GetFunctionName(), EventTrack.GetTrackName(), EventSigFunc->GetFName());
		}

		// Generate Update Pin Event Node
		CreatePinEventNodeForTimelineFunction(TimelineNode, SourceGraph, Timeline->GetUpdateFunctionName(), TEXT("Update"), EventSigFunc->GetFName());

		// Generate Finished Pin Event Node
		CreatePinEventNodeForTimelineFunction(TimelineNode, SourceGraph, Timeline->GetFinishedFunctionName(), TEXT("Finished"), EventSigFunc->GetFName());
	}
}

FPinConnectionResponse FKismetCompilerContext::MovePinLinksToIntermediate(UEdGraphPin& SourcePin, UEdGraphPin& IntermediatePin)
{
	FPinConnectionResponse ConnectionResult;

	// If we're modifying a removed pin there will be other compile errors and we don't want odd connection disallowed error so don't even try to move the pin links
	if (!SourcePin.bOrphanedPin)
	{
		UEdGraphSchema_K2 const* K2Schema = GetSchema();
		ConnectionResult = K2Schema->MovePinLinks(SourcePin, IntermediatePin, true);

		CheckConnectionResponse(ConnectionResult, SourcePin.GetOwningNode());
		MessageLog.NotifyIntermediatePinCreation(&IntermediatePin, &SourcePin);
	}

	 return ConnectionResult;
}

FPinConnectionResponse FKismetCompilerContext::CopyPinLinksToIntermediate(UEdGraphPin& SourcePin, UEdGraphPin& IntermediatePin)
{
	FPinConnectionResponse ConnectionResult;

	// If we're modifying a removed pin there will be other compile errors and we don't want odd connection disallowed error so don't even try to move the pin links
	if (!SourcePin.bOrphanedPin)
	{
		UEdGraphSchema_K2 const* K2Schema = GetSchema();
		ConnectionResult = K2Schema->CopyPinLinks(SourcePin, IntermediatePin, true);

		CheckConnectionResponse(ConnectionResult, SourcePin.GetOwningNode());
		MessageLog.NotifyIntermediatePinCreation(&IntermediatePin, &SourcePin);
	}

	return ConnectionResult;
}

UK2Node_TemporaryVariable* FKismetCompilerContext::SpawnInternalVariable(UEdGraphNode* SourceNode, const FName Category, const FName SubCategory, UObject* SubcategoryObject, EPinContainerType PinContainerType, const FEdGraphTerminalType& ValueTerminalType)
{
	UK2Node_TemporaryVariable* Result = SpawnIntermediateNode<UK2Node_TemporaryVariable>(SourceNode);

	Result->VariableType = FEdGraphPinType(Category, SubCategory, SubcategoryObject, PinContainerType, false, ValueTerminalType);
	Result->AllocateDefaultPins();

	return Result;
}

FName FKismetCompilerContext::GetEventStubFunctionName(UK2Node_Event* SrcEventNode)
{
	FName EventNodeName;

	// If we are overriding a function, we use the exact name for the event node
	if (SrcEventNode->bOverrideFunction)
	{
		EventNodeName = SrcEventNode->EventReference.GetMemberName();
	}
	else
	{
		// If not, create a new name
		if (SrcEventNode->CustomFunctionName != NAME_None)
		{
			EventNodeName = SrcEventNode->CustomFunctionName;
		}
		else
		{
			FString EventNodeString = ClassScopeNetNameMap.MakeValidName(SrcEventNode);
			EventNodeName = FName(*EventNodeString);
		}
	}

	return EventNodeName;
}

void FKismetCompilerContext::RegisterClassDelegateProxiesFromBlueprint()
{
	check(Blueprint);

	ConvertibleDelegates.Empty();

	for (const TObjectPtr<UEdGraph>& Graph : Blueprint->FunctionGraphs)
	{
		RegisterConvertibleDelegates(Graph);
	}

	for (const TObjectPtr<UEdGraph>& Graph : Blueprint->UbergraphPages)
	{
		RegisterConvertibleDelegates(Graph);
	}
}

void FKismetCompilerContext::CreateFunctionStubForEvent(UK2Node_Event* SrcEventNode, UObject* OwnerOfTemporaries)
{
	FName EventNodeName = GetEventStubFunctionName(SrcEventNode);

	// Create the stub graph and add it to the list of functions to compile

	FName EventNodeObjectName = EventNodeName;
	const UObject* ExistingObject = static_cast<UObject*>(FindObjectWithOuter(OwnerOfTemporaries, nullptr, EventNodeName));
	if (ExistingObject)
	{
		if (ExistingObject->IsA<UEdGraph>() && !ExistingObject->HasAnyFlags(RF_Transient))
		{
			MessageLog.Error(
				*FText::Format(
					LOCTEXT("CannotCreateStubForEvent_ErrorFmt", "Graph named '{0}' already exists in '{1}'. Another one cannot be generated from @@"),
					FText::FromName(EventNodeName),
					FText::FromString(*GetNameSafe(OwnerOfTemporaries))
				).ToString(),
				SrcEventNode
			);
			return;
		}
		else
		{
			// If we collide with another non-transient object type or any child object owned by the same object that will also contain transient
			// intermediate build artifacts post-compile, choose a unique name for the stub node. It's possible that the compile can still fail
			// due to collisions between generated properties and/or functions within the compiled class object, but choosing a unique name here
			// will avoid any issues with allocating the temporary node object below for the intermediate graph, so we at least get to that point.
			// 
			// @todo - Consider using a transient owner for build artifacts to avoid collisions with non-transient siblings that aren't going to
			// otherwise get cleared out on compile.
			EventNodeObjectName = MakeUniqueObjectName(OwnerOfTemporaries, UEdGraph::StaticClass(), EventNodeName);
		}
	}

	UEdGraph* ChildStubGraph = NewObject<UEdGraph>(OwnerOfTemporaries, EventNodeObjectName);
	Blueprint->EventGraphs.Add(ChildStubGraph);
	ChildStubGraph->Schema = UEdGraphSchema_K2::StaticClass();
	ChildStubGraph->SetFlags(RF_Transient);
	MessageLog.NotifyIntermediateObjectCreation(ChildStubGraph, SrcEventNode);

	FKismetFunctionContext& StubContext = *new FKismetFunctionContext(MessageLog, Schema, NewClass, Blueprint);
	FunctionList.Add(&StubContext);
	StubContext.SourceGraph = ChildStubGraph;

	StubContext.SourceEventFromStubGraph = SrcEventNode;

	if (SrcEventNode->bOverrideFunction || SrcEventNode->bInternalEvent)
	{
		StubContext.MarkAsInternalOrCppUseOnly();
	}

	uint32 FunctionFlags = SrcEventNode->FunctionFlags;
	if(SrcEventNode->bOverrideFunction && Blueprint->ParentClass != nullptr)
	{
		const UFunction* ParentFunction = Blueprint->ParentClass->FindFunctionByName(SrcEventNode->GetFunctionName());
		if(ParentFunction != nullptr)
		{
			FunctionFlags |= ParentFunction->FunctionFlags & FUNC_NetFuncFlags;
		}
	}

	if ((FunctionFlags & FUNC_Net) > 0)
	{
		StubContext.MarkAsNetFunction(FunctionFlags);
	}

	// Create an entry point
	UK2Node_FunctionEntry* EntryNode = SpawnIntermediateNode<UK2Node_FunctionEntry>(SrcEventNode, ChildStubGraph);
	EntryNode->NodePosX = -200;
	EntryNode->FunctionReference = SrcEventNode->EventReference;
	EntryNode->CustomGeneratedFunctionName = EventNodeName;

	// Resolve expansions to original custom event node before checking it for a server-only delegate association.
	auto IsServerOnlyEvent = [&MessageLog = this->MessageLog](UK2Node_Event* TargetEventNode)
	{
		if (UK2Node_CustomEvent* CustomEventNode = Cast<UK2Node_CustomEvent>(MessageLog.FindSourceObject(TargetEventNode)))
		{
			TargetEventNode = CustomEventNode;
		}

		return TargetEventNode->IsUsedByAuthorityOnlyDelegate();
	};

	if (!SrcEventNode->bOverrideFunction && IsServerOnlyEvent(SrcEventNode))
	{
		EntryNode->AddExtraFlags(FUNC_BlueprintAuthorityOnly);
	}

	// If this is a customizable event, make sure to copy over the user defined pins
	if (UK2Node_CustomEvent const* SrcCustomEventNode = Cast<UK2Node_CustomEvent const>(SrcEventNode))
	{
		EntryNode->UserDefinedPins = SrcCustomEventNode->UserDefinedPins;
		// CustomEvents may inherit net flags (so let's use their GetNetFlags() incase this is an override)
		StubContext.MarkAsNetFunction(SrcCustomEventNode->GetNetFlags());
		// Synchronize the entry node call in editor value with the entry point
		EntryNode->MetaData.bCallInEditor = SrcCustomEventNode->bCallInEditor;
		// Synchronize the node deprecation state with the entry point
		EntryNode->MetaData.bIsDeprecated = SrcCustomEventNode->bIsDeprecated;
		EntryNode->MetaData.DeprecationMessage = SrcCustomEventNode->DeprecationMessage;
	}
	EntryNode->AllocateDefaultPins();

	// Confirm that the event node matches the latest function signature, which the newly created EntryNode should have
	if( !SrcEventNode->IsFunctionEntryCompatible(EntryNode) )
	{
		// There is no match, so the function parameters must have changed.  Throw an error, and force them to refresh
		MessageLog.Error(*LOCTEXT("EventNodeOutOfDate_Error", "Event node @@ is out-of-date.  Please refresh it.").ToString(), SrcEventNode);
		return;
	}

	// Copy each event parameter to the assignment node, if there are any inputs
	UK2Node* AssignmentNode = NULL;
	for (int32 PinIndex = 0; PinIndex < EntryNode->Pins.Num(); ++PinIndex)
	{
		UEdGraphPin* SourcePin = EntryNode->Pins[PinIndex];
		if (!Schema->IsMetaPin(*SourcePin) && (SourcePin->Direction == EGPD_Output))
		{
			if (AssignmentNode == NULL)
			{
				// Create a variable write node to store the parameters into the ubergraph frame storage
				if (UsePersistentUberGraphFrame())
				{
					AssignmentNode = SpawnIntermediateNode<UK2Node_SetVariableOnPersistentFrame>(SrcEventNode, ChildStubGraph);
				}
				else
				{
					UK2Node_VariableSet* VariableSetNode = SpawnIntermediateNode<UK2Node_VariableSet>(SrcEventNode, ChildStubGraph);
					VariableSetNode->VariableReference.SetSelfMember(NAME_None);
					AssignmentNode = VariableSetNode;
				}
				check(AssignmentNode);
				AssignmentNode->AllocateDefaultPins();
			}

			// Determine what the member variable name is for this pin
			UEdGraphPin* UGSourcePin = SrcEventNode->FindPin(SourcePin->PinName);
			const FString MemberVariableName = ClassScopeNetNameMap.MakeValidName(UGSourcePin);

			UEdGraphPin* DestPin = AssignmentNode->CreatePin(EGPD_Input, SourcePin->PinType, *MemberVariableName);
			MessageLog.NotifyIntermediatePinCreation(DestPin, SourcePin);
			DestPin->MakeLinkTo(SourcePin);
		}
	}

	if (AssignmentNode == NULL)
	{
		// The event took no parameters, store it as a direct-access call
		StubContext.bIsSimpleStubGraphWithNoParams = true;
	}

	// Create a call into the ubergraph
	UK2Node_CallFunction* CallIntoUbergraph = SpawnIntermediateNode<UK2Node_CallFunction>(SrcEventNode, ChildStubGraph);
	CallIntoUbergraph->NodePosX = 300;

	// Use the ExecuteUbergraph base function to generate the pins...
	CallIntoUbergraph->FunctionReference.SetExternalMember(UEdGraphSchema_K2::FN_ExecuteUbergraphBase, UObject::StaticClass());
	CallIntoUbergraph->AllocateDefaultPins();
	
	// ...then swap to the generated version for this level
	CallIntoUbergraph->FunctionReference.SetSelfMember(GetUbergraphCallName());
	UEdGraphPin* CallIntoUbergraphSelf = Schema->FindSelfPin(*CallIntoUbergraph, EGPD_Input);
	CallIntoUbergraphSelf->PinType.PinSubCategory = UEdGraphSchema_K2::PSC_Self;
	CallIntoUbergraphSelf->PinType.PinSubCategoryObject = *Blueprint->SkeletonGeneratedClass;

	UEdGraphPin* EntryPointPin = CallIntoUbergraph->FindPin(UEdGraphSchema_K2::PN_EntryPoint);
	if (EntryPointPin)
	{
		EntryPointPin->DefaultValue = TEXT("0");
	}

	// Schedule a patchup on the event entry address
	CallsIntoUbergraph.Add(CallIntoUbergraph, SrcEventNode);

	// Wire up the node execution wires
	UEdGraphPin* ExecEntryOut = Schema->FindExecutionPin(*EntryNode, EGPD_Output);
	UEdGraphPin* ExecCallIn = Schema->FindExecutionPin(*CallIntoUbergraph, EGPD_Input);

	if (AssignmentNode)
	{
		UEdGraphPin* ExecVariablesIn = Schema->FindExecutionPin(*AssignmentNode, EGPD_Input);
		UEdGraphPin* ExecVariablesOut = Schema->FindExecutionPin(*AssignmentNode, EGPD_Output);

		ExecEntryOut->MakeLinkTo(ExecVariablesIn);
		ExecVariablesOut->MakeLinkTo(ExecCallIn);
	}
	else
	{
		ExecEntryOut->MakeLinkTo(ExecCallIn);
	}
}

void FKismetCompilerContext::MergeUbergraphPagesIn(UEdGraph* Ubergraph)
{
	BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_MergeUbergraphPagesIn);
	
	for (decltype(Blueprint->UbergraphPages)::TIterator It(Blueprint->UbergraphPages); It; ++It)
	{
		UEdGraph* SourceGraph = *It;
		MergeGraphIntoUbergraph(SourceGraph, Ubergraph);
	}

	for (UEdGraph* UbergraphPage : GeneratedUbergraphPages)
	{
		MergeGraphIntoUbergraph(UbergraphPage, Ubergraph);
	}
}

// Expands out nodes that need it
void FKismetCompilerContext::ExpansionStep(UEdGraph* Graph, bool bAllowUbergraphExpansions)
{
	auto PruneInner = [this, Graph]()
	{
		// Find the connected subgraph starting at the root node and prune out unused nodes
		const bool bIncludePotentialRootNodes = true;
		PruneIsolatedNodes(Graph, bIncludePotentialRootNodes);
	};

	// Node expansion may affect the signature of a static function
	if (bIsFullCompile)
	{
		BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_Expansion);

		// First we need to expand knot nodes, so it will remote disconnected knots
		// Collapse any remaining tunnels or macros
		ExpandTunnelsAndMacros(Graph);

		// First pruning pass must be call after all collapsed nodes are expanded. Before the expansion we don't know which collapsed graph is really isolated. 
		// If the pruning was called before expansion (and all collapsed graphs were saved), the isolated collapsed graphs would be unnecessarily validated.
		PruneInner();

		// First we need to expand knot nodes so any other expansions like AutoCreateRefTerm will have the correct pins hooked up
		for (int32 NodeIndex = 0; NodeIndex < Graph->Nodes.Num(); ++NodeIndex)
		{
			UK2Node_Knot* KnotNode = Cast<UK2Node_Knot>(Graph->Nodes[NodeIndex]);
			if (KnotNode)
			{
				BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_ExpandNode);
				KnotNode->ExpandNode(*this, Graph);
			}
		}

		for (int32 NodeIndex = 0; NodeIndex < Graph->Nodes.Num(); ++NodeIndex)
		{
			UK2Node* Node = Cast<UK2Node>(Graph->Nodes[NodeIndex]);
			if (Node)
			{
				BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_ExpandNode);
				Node->ExpandNode(*this, Graph);
			}
		}
	}
	else
	{
		PruneInner();
	}

	if (bAllowUbergraphExpansions)
	{
		// Expand timeline nodes, in skeleton classes only the events will be generated
		ExpandTimelineNodes(Graph);
	}

	{
		BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_PostExpansionStep);
		PostExpansionStep(Graph);
	}
}

void FKismetCompilerContext::DetermineNodeExecLinks(UEdGraphNode* SourceNode, TMap<UEdGraphPin*, UEdGraphPin*>& SourceNodeLinks) const
{
	// Find all linked pins we care about from the source node.
	for (UEdGraphPin* SourcePin : SourceNode->Pins)
	{
		if (SourcePin->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec)
		{
			UEdGraphPin* TrueSourcePin = MessageLog.FindSourcePin(SourcePin);
			for (UEdGraphPin* LinkedPin : SourcePin->LinkedTo)
			{
				SourceNodeLinks.Add(LinkedPin, TrueSourcePin);
			}
		}
	}
}

void FKismetCompilerContext::CreateLocalsAndRegisterNets(FKismetFunctionContext& Context, FField::FLinkedListBuilder& PropertyListBuilder)
{
	BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_CreateLocalsAndRegisterNets);

	// Create any user defined variables, this must occur before registering nets so that the properties are in place
	CreateUserDefinedLocalVariablesForFunction(Context, PropertyListBuilder);

	check(Context.IsValid());
	//@TODO: Prune pure functions that don't have any consumers
	if (bIsFullCompile)
	{
		// Find the execution path (and make sure it has no cycles)
		CreateExecutionSchedule(Context.SourceGraph->Nodes, Context.LinearExecutionList);

		// Register nets for any nodes still in the schedule (as long as they didn't get registered in the initial all-nodes pass)
		for (UEdGraphNode* Node : Context.LinearExecutionList)
		{
			if (FNodeHandlingFunctor* Handler = NodeHandlers.FindRef(Node->GetClass()))
			{
				if (!Handler->RequiresRegisterNetsBeforeScheduling())
				{
					Handler->RegisterNets(Context, Node);
				}
			}
			else
			{
				MessageLog.Error(
					*FText::Format(
						LOCTEXT("UnexpectedNodeType_ErrorFmt", "Unexpected node type {0} encountered at @@"),
						FText::FromString(Node->GetClass()->GetName())
					).ToString(),
					Node
				);
			}
		}
	}

	using namespace UE::KismetCompiler;

	CastingUtils::RegisterImplicitCasts(Context);

	// Create net variable declarations
	CreateLocalVariablesForFunction(Context, PropertyListBuilder);
}

void FKismetCompilerContext::VerifyValidOverrideEvent(const UEdGraph* Graph)
{
	check(NULL != Graph);
	check(NULL != Blueprint);

	TArray<const UK2Node_Event*> EntryPoints;
	Graph->GetNodesOfClass(EntryPoints);

	for (TFieldIterator<UFunction> FunctionIt(Blueprint->ParentClass, EFieldIteratorFlags::IncludeSuper); FunctionIt; ++FunctionIt)
	{
		const UFunction* Function = *FunctionIt;
		if(!UEdGraphSchema_K2::FunctionCanBePlacedAsEvent(Function))
		{
			const UClass* FuncClass = CastChecked<UClass>(Function->GetOuter());
			const FName FuncName = Function->GetFName();
			for(int32 EntryPointsIdx = 0; EntryPointsIdx < EntryPoints.Num(); EntryPointsIdx++)
			{
				const UK2Node_Event* EventNode = EntryPoints[EntryPointsIdx];
				if( EventNode && EventNode->bOverrideFunction &&
					(EventNode->EventReference.GetMemberParentClass(EventNode->GetBlueprintClassFromNode()) == FuncClass) &&
					(EventNode->EventReference.GetMemberName() == FuncName))
				{
					if (EventNode->HasDeprecatedReference())
					{
						// The event cannot be placed because it has been deprecated. However, we already emit a
						// warning in FGraphCompilerContext::ValidateNode(), so there's no need to repeat it here.
						continue;
					}
					else if(!Function->HasAllFunctionFlags(FUNC_Const))	// ...allow legacy event nodes that override methods declared as 'const' to pass.
					{
						MessageLog.Error(TEXT("The function in node @@ cannot be overridden and/or placed as event"), EventNode);
					}
				}
			}
		}
	}
}

void FKismetCompilerContext::VerifyValidOverrideFunction(const UEdGraph* Graph)
{
	check(nullptr != Graph);
	check(nullptr != Blueprint);

	TArray<UK2Node_FunctionEntry*> EntryPoints;
	Graph->GetNodesOfClass(EntryPoints);

	for(int32 EntryPointsIdx = 0; EntryPointsIdx < EntryPoints.Num(); EntryPointsIdx++)
	{
		UK2Node_FunctionEntry* EntryNode = EntryPoints[EntryPointsIdx];
		check(nullptr != EntryNode);

		const UClass* FuncClass = EntryNode->FunctionReference.GetMemberParentClass();
		if (FuncClass)
		{
			const UFunction* Function = FuncClass->FindFunctionByName(EntryNode->FunctionReference.GetMemberName());
			if (Function)
			{
				const bool bCanBeOverridden = Function->HasAllFunctionFlags(FUNC_BlueprintEvent);
				if (!bCanBeOverridden)
				{
					MessageLog.Error(TEXT("The function in node @@ cannot be overridden"), EntryNode);
				}
			}
		}
		else
		{
			// check if the function name is unique
			for (TFieldIterator<UFunction> FunctionIt(Blueprint->ParentClass, EFieldIteratorFlags::IncludeSuper); FunctionIt; ++FunctionIt)
			{
				const UFunction* Function = *FunctionIt;
				if (Function && (Function->GetFName() == EntryNode->FunctionReference.GetMemberName()))
				{
					MessageLog.Error(TEXT("The function name in node @@ is already used"), EntryNode);
				}
			}
		}
	}
}

namespace UE::Private
{

static bool CanEventGraphBePruned(const UEdGraph* ConsolidatedEventGraph)
{
	static const FBoolConfigValueHelper DisableLateEventGraphPruning(TEXT("Blueprints"), TEXT("bDisableLateEventGraphPruning"), GEngineIni);
	if(DisableLateEventGraphPruning)
	{
		return false;
	}

	return Algo::FindByPredicate(ConsolidatedEventGraph->Nodes, 
		[](const UEdGraphNode* Node)
		{
			return Cast<UK2Node_Event>(Node) != nullptr;
		}) == nullptr;
}

}

// Merges pages and creates function stubs, etc... from the ubergraph entry points
void FKismetCompilerContext::CreateAndProcessUbergraph()
{
	BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_ProcessUbergraph);

	ConsolidatedEventGraph = NewObject<UEdGraph>(Blueprint, GetUbergraphCallName());
	ConsolidatedEventGraph->Schema = UEdGraphSchema_K2::StaticClass();
	ConsolidatedEventGraph->SetFlags(RF_Transient);

	// Merge all of the top-level pages
	MergeUbergraphPagesIn(ConsolidatedEventGraph);

	// Loop over implemented interfaces, and add dummy event entry points for events that aren't explicitly handled by the user
	TArray<UK2Node_Event*> EntryPoints;
	ConsolidatedEventGraph->GetNodesOfClass(EntryPoints);

	for (int32 i = 0; i < Blueprint->ImplementedInterfaces.Num(); i++)
	{
		const FBPInterfaceDescription& InterfaceDesc = Blueprint->ImplementedInterfaces[i];
		for (TFieldIterator<UFunction> FunctionIt(InterfaceDesc.Interface, EFieldIteratorFlags::IncludeSuper); FunctionIt; ++FunctionIt)
		{
			const UFunction* Function = *FunctionIt;
			const FName FunctionName = Function->GetFName();

			const bool bCanImplementAsEvent = UEdGraphSchema_K2::FunctionCanBePlacedAsEvent(Function);
			bool bExistsAsGraph = false;

			// Any function that can be implemented as an event needs to check to see if there is already an interface function graph
			if (bCanImplementAsEvent)
			{
				for (UEdGraph* InterfaceGraph : InterfaceDesc.Graphs)
				{
					if (InterfaceGraph->GetFName() == Function->GetFName())
					{
						bExistsAsGraph = true;
					}
				}
			}

			// If this is an event, check the merged ubergraph to make sure that it has an event handler, and if not, add one
			if (bCanImplementAsEvent && UEdGraphSchema_K2::CanKismetOverrideFunction(Function) && !bExistsAsGraph)
			{
				bool bFoundEntry = false;
				// Search the cached entry points to see if we have a match
				for (int32 EntryIndex = 0; EntryIndex < EntryPoints.Num(); ++EntryIndex)
				{
					const UK2Node_Event* EventNode = EntryPoints[EntryIndex];
					if( EventNode && (EventNode->EventReference.GetMemberName() == FunctionName) )
					{
						bFoundEntry = true;
						break;
					}
				}

				if (!bFoundEntry)
				{
					// Create an entry node stub, so that we have a entry point for interfaces to call to
					UK2Node_Event* EventNode = SpawnIntermediateNode<UK2Node_Event>(nullptr, ConsolidatedEventGraph);
					EventNode->EventReference.SetExternalMember(FunctionName, InterfaceDesc.Interface);
					EventNode->bOverrideFunction = true;
					EventNode->AllocateDefaultPins();
				}
			}
		}
	}

	// We need to stop the old EventGraphs from having the Blueprint as an outer, it impacts renaming.
	if(!Blueprint->HasAnyFlags(RF_NeedLoad|RF_NeedPostLoad))
	{
		for(UEdGraph* OldEventGraph : Blueprint->EventGraphs)
		{
			if (OldEventGraph)
			{
				OldEventGraph->Rename(NULL, GetTransientPackage());
			}
		}
	}
	Blueprint->EventGraphs.Empty();

	if (ConsolidatedEventGraph->Nodes.Num())
	{
		// Add a dummy entry point to the uber graph, to get the function signature correct
		{
			UK2Node_FunctionEntry* EntryNode = SpawnIntermediateNode<UK2Node_FunctionEntry>(NULL, ConsolidatedEventGraph);
			EntryNode->FunctionReference.SetExternalMember(UEdGraphSchema_K2::FN_ExecuteUbergraphBase, UObject::StaticClass());
			EntryNode->CustomGeneratedFunctionName = ConsolidatedEventGraph->GetFName();
			EntryNode->AllocateDefaultPins();
		}

		// Expand out nodes that need it
		ExpansionStep(ConsolidatedEventGraph, true);

		ReplaceConvertibleDelegates(ConsolidatedEventGraph);

		// If a function in the graph cannot be overridden/placed as event make sure that it is not.
		VerifyValidOverrideEvent(ConsolidatedEventGraph);
		
		// expansion may have removed all of the event entry points, in which case nothing will
		// be placed in the ubergraph function and we can prune it:
		if(UE::Private::CanEventGraphBePruned(ConsolidatedEventGraph))
		{
			return;
		}

		// Do some cursory validation (pin types match, inputs to outputs, pins never point to their parent node, etc...)
		{
			BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_Validate);

			UbergraphContext = new FKismetFunctionContext(MessageLog, Schema, NewClass, Blueprint);
			FunctionList.Add(UbergraphContext);
			UbergraphContext->SourceGraph = ConsolidatedEventGraph;
			UbergraphContext->MarkAsEventGraph();
			UbergraphContext->MarkAsInternalOrCppUseOnly();
			UbergraphContext->SetExternalNetNameMap(&ClassScopeNetNameMap);

			// Validate all the nodes in the graph
			for (int32 ChildIndex = 0; ChildIndex < ConsolidatedEventGraph->Nodes.Num(); ++ChildIndex)
			{
				const UEdGraphNode* Node = ConsolidatedEventGraph->Nodes[ChildIndex];
				const int32 SavedErrorCount = MessageLog.NumErrors;
				UK2Node_Event* SrcEventNode = Cast<UK2Node_Event>(ConsolidatedEventGraph->Nodes[ChildIndex]);
				if (bIsFullCompile)
				{
					// We only validate a full compile, we want to always make a function stub so we can display the errors for it later
					ValidateNode(Node);
				}

				// If the node didn't generate any errors then generate function stubs for event entry nodes etc.
				if ((SavedErrorCount == MessageLog.NumErrors) && SrcEventNode)
				{
					CreateFunctionStubForEvent(SrcEventNode, Blueprint);
				}
			}
		}
	}
}

void FKismetCompilerContext::AutoAssignNodePosition(UEdGraphNode* Node)
{
	int32 Width = FMath::Max<int32>(Node->NodeWidth, AverageNodeWidth);
	int32 Height = FMath::Max<int32>(Node->NodeHeight, AverageNodeHeight);

	Node->NodePosX = MacroSpawnX;
	Node->NodePosY = MacroSpawnY;

	MacroSpawnX += Width + HorizontalNodePadding;
	MacroRowMaxHeight = FMath::Max<int32>(MacroRowMaxHeight, Height);

	// Advance the spawn position
	if (MacroSpawnX >= MaximumSpawnX)
	{
		MacroSpawnX = MinimumSpawnX;
		MacroSpawnY += MacroRowMaxHeight + VerticalSectionPadding;

		MacroRowMaxHeight = 0;
	}
}

void FKismetCompilerContext::AdvanceMacroPlacement(int32 Width, int32 Height)
{
	MacroSpawnX += Width + HorizontalSectionPadding;
	MacroRowMaxHeight = FMath::Max<int32>(MacroRowMaxHeight, Height);

	if (MacroSpawnX > MaximumSpawnX)
	{
		MacroSpawnX = MinimumSpawnX;
		MacroSpawnY += MacroRowMaxHeight + VerticalSectionPadding;

		MacroRowMaxHeight = 0;
	}
}

void FKismetCompilerContext::CreateCommentBlockAroundNodes(const TArray<UEdGraphNode*>& Nodes, UObject* SourceObject, UEdGraph* TargetGraph, FString CommentText, FLinearColor CommentColor, int32& Out_OffsetX, int32& Out_OffsetY)
{
	if (!Nodes.Num())
	{
		return;
	}

	FIntRect Bounds = FEdGraphUtilities::CalculateApproximateNodeBoundaries(Nodes);

	// Figure out how to offset the expanded nodes to fit into our tile
	Out_OffsetX = MacroSpawnX - Bounds.Min.X;
	Out_OffsetY = MacroSpawnY - Bounds.Min.Y;

	// Create a comment node around the expanded nodes, using the name
	const int32 Padding = 60;

	UEdGraphNode_Comment* CommentNode = SpawnIntermediateNode<UEdGraphNode_Comment>(Cast<UEdGraphNode>(SourceObject), TargetGraph);
	CommentNode->CommentColor = CommentColor;
	CommentNode->NodePosX = MacroSpawnX - Padding;
	CommentNode->NodePosY = MacroSpawnY - Padding;
	CommentNode->NodeWidth = Bounds.Width() + 2*Padding;
	CommentNode->NodeHeight = Bounds.Height() + 2*Padding;
	CommentNode->NodeComment = CommentText;
	CommentNode->AllocateDefaultPins();

	// Advance the macro expansion tile to the next open slot
	AdvanceMacroPlacement(Bounds.Width(), Bounds.Height());
}

void FKismetCompilerContext::ExpandTunnelsAndMacros(UEdGraph* SourceGraph)
{
	check(SourceGraph);

	// Macro cycles can lead to infinite creation of nodes and hang the editor.
	if (UE::KismetCompiler::Private::FindAndReportMacroCycles(SourceGraph, MessageLog))
	{
		return;
	}

	// Determine if we are regenerating a blueprint on load
	const bool bIsLoading = Blueprint ? Blueprint->bIsRegeneratingOnLoad : false;

	// Collapse any remaining tunnels
	for (decltype(SourceGraph->Nodes)::TIterator NodeIt(SourceGraph->Nodes); NodeIt; ++NodeIt)
	{
		UEdGraphNode* CurrentNode = *NodeIt;
		if (!CurrentNode || !CurrentNode->ShouldMergeChildGraphs())
		{
			continue;
		}

		UK2Node_Tunnel* TunnelNode = Cast<UK2Node_Tunnel>(*NodeIt);
		// After this expansion (and before the validation) PruneIsolatedNodes is called. So this is the last chance to validate nodes like UK2Node_MathExpression.
		// Notice: even isolated MathExpression nodes will be validated. But, since the MathExpression is usually optimized (so it is not handled here as tunnel, because ShouldMergeChildGraphs return false) it is not a problem.
		// Notice: MacroInstance Node is based on Tunnel Node.
		if (TunnelNode)
		{
			TunnelNode->ValidateNodeDuringCompilation(MessageLog);
		}

		if (UK2Node_MacroInstance* MacroInstanceNode = Cast<UK2Node_MacroInstance>(*NodeIt))
		{
			UEdGraph* MacroGraph = MacroInstanceNode->GetMacroGraph();
			// Verify that this macro can actually be expanded
			if (MacroGraph == nullptr)
			{
				MessageLog.Error(TEXT("Macro node @@ is pointing at an invalid macro graph."), MacroInstanceNode);
				continue;
			}

			UBlueprint* MacroBlueprint = FBlueprintEditorUtils::FindBlueprintForGraph(MacroGraph);
			// unfortunately, you may be expanding a macro that has yet to be
			// regenerated on load (thanks cyclic dependencies!), and in certain
			// cases the nodes found within the macro may be out of date 
			// (function signatures, etc.), so let's force a reconstruct of the 
			// nodes we inject from the macro (just in case).
			const bool bForceRegenNodes = bIsLoading && MacroBlueprint && (MacroBlueprint != Blueprint) && !MacroBlueprint->bHasBeenRegenerated;

			// Clone the macro graph, then move all of its children, keeping a list of nodes from the macro
			UEdGraph* ClonedGraph = FEdGraphUtilities::CloneGraph(MacroGraph, nullptr, &MessageLog, true);

			for (int32 I = 0; I < ClonedGraph->Nodes.Num(); ++I)
			{
				MacroGeneratedNodes.Add(ClonedGraph->Nodes[I], CurrentNode);
				MessageLog.NotifyIntermediateMacroNode(CurrentNode, ClonedGraph->Nodes[I]);
			}

			TArray<UEdGraphNode*> MacroNodes(ClonedGraph->Nodes);

			// Handle any nodes that need to inherit their macro instance's NodeGUID
			for( UEdGraphNode* ClonedNode : MacroNodes)
			{
				UK2Node_TemporaryVariable* TempVarNode = Cast<UK2Node_TemporaryVariable>(ClonedNode);
				if(TempVarNode && TempVarNode->bIsPersistent)
				{
					TempVarNode->NodeGuid = MacroInstanceNode->NodeGuid;
				}
			}

			// We iterate the array in reverse so we can both remove the subpins safely after we've read them and
			// so we have split nested structs we combine them back together in the right order
			for (int32 PinIndex = MacroInstanceNode->Pins.Num() - 1; PinIndex >= 0; --PinIndex)
			{
				UEdGraphPin* Pin = MacroInstanceNode->Pins[PinIndex];
				if (Pin)
				{
					// Since we don't support array literals, drop a make array node on any unconnected array pins, which will allow macro expansion to succeed even if disconnected
					if (Pin->PinType.IsArray()
					&& (Pin->Direction == EGPD_Input)
					&& (Pin->LinkedTo.Num() == 0))
					{
						UK2Node_MakeArray* MakeArrayNode = SpawnIntermediateNode<UK2Node_MakeArray>(MacroInstanceNode, SourceGraph);
						MakeArrayNode->NumInputs = 0; // the generated array should be empty
						MakeArrayNode->AllocateDefaultPins();
						UEdGraphPin* MakeArrayOut = MakeArrayNode->GetOutputPin();
						check(MakeArrayOut);
						MakeArrayOut->MakeLinkTo(Pin);
						MakeArrayNode->PinConnectionListChanged(MakeArrayOut);
					}
					else if (Pin->LinkedTo.Num() == 0 &&
							Pin->Direction == EGPD_Input &&
							Pin->DefaultValue != FString() &&
							Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Byte &&
							Pin->PinType.PinSubCategoryObject.IsValid() &&
							Pin->PinType.PinSubCategoryObject->IsA<UEnum>())
					{
						// Similarly, enums need a 'make enum' node because they decay to byte after instantiation:
						UK2Node_EnumLiteral* EnumLiteralNode = SpawnIntermediateNode<UK2Node_EnumLiteral>(MacroInstanceNode, SourceGraph);
						EnumLiteralNode->Enum = CastChecked<UEnum>(Pin->PinType.PinSubCategoryObject.Get());
						EnumLiteralNode->AllocateDefaultPins();
						EnumLiteralNode->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue)->MakeLinkTo(Pin);

						UEdGraphPin* InPin = EnumLiteralNode->FindPinChecked(UK2Node_EnumLiteral::GetEnumInputPinName());
						check(InPin);
						InPin->DefaultValue = Pin->DefaultValue;
					}
					// Otherwise we need to handle the pin splitting
					else if (Pin->SubPins.Num() > 0)
					{
						MacroInstanceNode->ExpandSplitPin(this, SourceGraph, Pin);
					}
				}
			}

			ClonedGraph->MoveNodesToAnotherGraph(SourceGraph, IsAsyncLoading() || bIsLoading, Blueprint && Blueprint->bBeingCompiled);
			FEdGraphUtilities::MergeChildrenGraphsIn(SourceGraph, ClonedGraph, /*bRequireSchemaMatch=*/ true);

			// run type inference on MacroNodes:
			MacroInstanceNode->InferWildcards(MacroNodes);

			// When emitting intermediate products; make an effort to make them readable by preventing overlaps and adding informative comments
			int32 NodeOffsetX = 0;
			int32 NodeOffsetY = 0;
			if (CompileOptions.bSaveIntermediateProducts)
			{
				CreateCommentBlockAroundNodes(
					MacroNodes,
					MacroInstanceNode,
					SourceGraph,
					FText::Format(LOCTEXT("ExpandedMacroCommentFmt", "Macro {0}"), FText::FromString(MacroGraph->GetName())).ToString(),
					MacroInstanceNode->MetaData.InstanceTitleColor,
					/*out*/ NodeOffsetX,
					/*out*/ NodeOffsetY);
			}

			// Record intermediate object creation nodes, offset the nodes, and handle tunnels
			for (TArray<UEdGraphNode*>::TIterator MacroNodeIt(MacroNodes); MacroNodeIt; ++MacroNodeIt)
			{
				UEdGraphNode* DuplicatedNode = *MacroNodeIt;
				
				if (DuplicatedNode != nullptr)
				{
					if (bForceRegenNodes)
					{
						DuplicatedNode->ReconstructNode();
					}

					DuplicatedNode->NodePosY += NodeOffsetY;
					DuplicatedNode->NodePosX += NodeOffsetX;

					if (const UK2Node_Composite* const CompositeNode = Cast<const UK2Node_Composite>(DuplicatedNode))
					{
						// Composite nodes can be present in the MacroNodes if users have collapsed nodes in the macro.
						// No need to do anything for those:
						continue;
					}
					
					if (UK2Node_Tunnel* DuplicatedTunnelNode = Cast<UK2Node_Tunnel>(DuplicatedNode))
					{
						// Tunnel nodes should be connected to the MacroInstance they have been instantiated by.. Note 
						// that if there are tunnel nodes internal to the macro instance they will be incorrectly 
						// connected to the MacroInstance.
						if (DuplicatedTunnelNode->bCanHaveInputs)
						{
							check(!DuplicatedTunnelNode->bCanHaveOutputs);
							// If this check fails it indicates that we've failed to identify all uses of tunnel nodes and 
							// are erroneously connecting tunnels to the macro instance when they should be left untouched..
							check(!DuplicatedTunnelNode->InputSinkNode);
							DuplicatedTunnelNode->InputSinkNode = MacroInstanceNode;
							MacroInstanceNode->OutputSourceNode = DuplicatedTunnelNode;
						}
						else if (DuplicatedTunnelNode->bCanHaveOutputs)
						{
							check(!DuplicatedTunnelNode->OutputSourceNode);
							DuplicatedTunnelNode->OutputSourceNode = MacroInstanceNode;
							MacroInstanceNode->InputSinkNode = DuplicatedTunnelNode;
						}
					}
				}
			}
		}
		else if (TunnelNode)
		{
			UK2Node_Tunnel* InputSink = TunnelNode->GetInputSink();
			UK2Node_Tunnel* OutputSource = TunnelNode->GetOutputSource();
			
			// Determine the tunnel nodes that bound the expansion
			UK2Node_Tunnel* TunnelInstance = nullptr;
			UK2Node_Tunnel* TunnelInputSite = nullptr;
			UK2Node_Tunnel *TunnelOutputSite = nullptr;
			if (FBlueprintEditorUtils::IsTunnelInstanceNode(TunnelNode))
			{
				TunnelInstance = TunnelNode;
				TunnelInputSite = InputSink;
				TunnelOutputSite = OutputSource;
			}
			else if (FBlueprintEditorUtils::IsTunnelInstanceNode(InputSink))
			{
				TunnelInstance = InputSink;
				TunnelOutputSite = TunnelNode;
			}
			else if (FBlueprintEditorUtils::IsTunnelInstanceNode(OutputSource))
			{
				TunnelInstance = OutputSource;
				TunnelInputSite = TunnelNode;
			}

			if (TunnelInstance)
			{
				if (TunnelInputSite)
				{
					// Construct an intermediate tunnel boundary on the input side of a tunnel instance expansion.
					ProcessIntermediateTunnelBoundary(TunnelInstance, TunnelInputSite);
				}

				if (TunnelOutputSite)
				{
					// Construct an intermediate tunnel boundary on the output side of a tunnel instance expansion.
					ProcessIntermediateTunnelBoundary(TunnelOutputSite, TunnelInstance);
				}
			}

			const bool bSuccess = Schema->CollapseGatewayNode(TunnelNode, InputSink, OutputSource, this);
			if (!bSuccess)
			{
				MessageLog.Error(*LOCTEXT("CollapseTunnel_Error", "Failed to collapse tunnel @@").ToString(), TunnelNode);
			}
		}
	}
}

void FKismetCompilerContext::ResetErrorFlags(UEdGraph* Graph) const
{
	if (Graph != NULL)
	{
		for (int32 NodeIndex = 0; NodeIndex < Graph->Nodes.Num(); ++NodeIndex)
		{
			if (UEdGraphNode* GraphNode = Graph->Nodes[NodeIndex])
			{
				GraphNode->ClearCompilerMessage();
			}
		}
	}
}

void FKismetCompilerContext::MergeGraphIntoUbergraph(UEdGraph* SourceGraph, UEdGraph* Ubergraph)
{
	if (CompileOptions.bSaveIntermediateProducts)
	{
		TArray<UEdGraphNode*> ClonedNodeList;
		FEdGraphUtilities::CloneAndMergeGraphIn(Ubergraph, SourceGraph, MessageLog, /*bRequireSchemaMatch=*/ true, /*bIsCompiling*/ true, &ClonedNodeList);

		// Create a comment block around the ubergrapgh contents before anything else got started
		int32 OffsetX = 0;
		int32 OffsetY = 0;
		CreateCommentBlockAroundNodes(ClonedNodeList, SourceGraph, Ubergraph, SourceGraph->GetName(), FLinearColor(1.0f, 0.7f, 0.7f), /*out*/ OffsetX, /*out*/ OffsetY);
			
		// Reposition the nodes, so nothing ever overlaps
		for (TArray<UEdGraphNode*>::TIterator NodeIt(ClonedNodeList); NodeIt; ++NodeIt)
		{
			UEdGraphNode* ClonedNode = *NodeIt;

			ClonedNode->NodePosX += OffsetX;
			ClonedNode->NodePosY += OffsetY;
		}
	}
	else
	{
		FEdGraphUtilities::CloneAndMergeGraphIn(Ubergraph, SourceGraph, MessageLog, /*bRequireSchemaMatch=*/ true, /*bIsCompiling*/ true);
	}
}

/**
 * Merges macros/subgraphs into the graph and validates it, creating a function list entry if it's reasonable.
 */
void FKismetCompilerContext::ProcessOneFunctionGraph(UEdGraph* SourceGraph, bool bInternalFunction)
{
	BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_ProcessFunctionGraph);

	if (SourceGraph->GetFName() == Schema->FN_UserConstructionScript && FBlueprintEditorUtils::IsDataOnlyBlueprint(Blueprint))
	{
		// This is a data only blueprint, we do not want to actually create our user construction script as it only consists of a call to the parent
		return;
	}

	// Clone the source graph so we can modify it as needed; merging in the child graphs
	UEdGraph* FunctionGraph = FEdGraphUtilities::CloneGraph(SourceGraph, Blueprint, &MessageLog, true);

	// Preserve the original graph name, which helps with debugging intermediate products
	{
		FString FunctionGraphName = SourceGraph->GetName() + TEXT("_MERGED");

		ERenameFlags RenameFlags =
			(REN_NonTransactional | REN_DoNotDirty | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);

		if (UEdGraph* ExistingGraph = FindObject<UEdGraph>(Blueprint, *FunctionGraphName))
		{
			ExistingGraph->Rename(nullptr, GetTransientPackage(), RenameFlags);
		}

		FunctionGraph->Rename(*FunctionGraphName, nullptr, RenameFlags);
	}

	const int32 SavedErrorCount = MessageLog.NumErrors;
	bool bIsInvalidFunctionGraph = false;
	FEdGraphUtilities::MergeChildrenGraphsIn(FunctionGraph, FunctionGraph, /* bRequireSchemaMatch = */ true, /* bInIsCompiling = */ true, &MessageLog);

	// If we failed to merge with any child graphs due to an error, we shouldn't continue processing the intermediate graph.
	if (MessageLog.NumErrors > SavedErrorCount)
	{
		bIsInvalidFunctionGraph = true;
	}
	else
	{
		ExpansionStep(FunctionGraph, false);

		ReplaceConvertibleDelegates(FunctionGraph);

		// Cull the entire construction script graph if after node culling it's trivial, this reduces event spam on object construction:
		if (SourceGraph->GetFName() == Schema->FN_UserConstructionScript )
		{
			if(FKismetCompilerUtilities::IsIntermediateFunctionGraphTrivial(Schema->FN_UserConstructionScript, FunctionGraph))
			{
				return;
			}
		}

		// If a function in the graph cannot be overridden/placed as event make sure that it is not.
		VerifyValidOverrideFunction(FunctionGraph);

		// NOTE: The Blueprint compilation manager generates the skeleton class using a different
		// code path. We do NOT want ValidateGraphIsWellFormed() ran for skeleton-only compiles here
		// because it can result in errors (the function hasn't been added to the class yet, etc.)
		check(CompileOptions.CompileType != EKismetCompileType::SkeletonOnly);

		// First do some cursory validation (pin types match, inputs to outputs, pins never point to their parent node, etc...)
		// If this fails we will "stub" the function graph (if callable) or stop altogether to avoid crashes or infinite loops
		bIsInvalidFunctionGraph = !ValidateGraphIsWellFormed(FunctionGraph);
	}

	if (bIsInvalidFunctionGraph)
	{
		if(bInternalFunction)
		{
			// Internal functions that are not well-formed can be culled, since they're not exposed or callable.
			return;
		}
		else
		{
			// Break all links to the entry point in the cloned graph to create a "stub" context that's still exposed
			// as a callable function. This way external dependencies can still rely on the public interface if they're
			// not themselves being fully recompiled as a dependency of this Blueprint class.
			TArray<UK2Node_FunctionEntry*> EntryNodes;
			FunctionGraph->GetNodesOfClass<UK2Node_FunctionEntry>(EntryNodes);
			for (UK2Node_FunctionEntry* EntryNode : EntryNodes)
			{
				if (EntryNode)
				{
					EntryNode->BreakAllNodeLinks();
				}
			}
		}
	}

	const UEdGraphSchema_K2* FunctionGraphSchema = CastChecked<const UEdGraphSchema_K2>(FunctionGraph->GetSchema());
	FKismetFunctionContext& Context = *new FKismetFunctionContext(MessageLog, FunctionGraphSchema, NewClass, Blueprint);
	FunctionList.Add(&Context);
	Context.SourceGraph = FunctionGraph;

	if (FBlueprintEditorUtils::IsDelegateSignatureGraph(SourceGraph)) //-V1051
	{
		Context.SetDelegateSignatureName(SourceGraph->GetFName());
	}

	// If this is an interface blueprint, mark the function contexts as stubs
	if (FBlueprintEditorUtils::IsInterfaceBlueprint(Blueprint))
	{
		Context.MarkAsInterfaceStub();
	}

	bool bEnforceConstCorrectness = true;
	if (FBlueprintEditorUtils::IsBlueprintConst(Blueprint) || Context.Schema->IsConstFunctionGraph(Context.SourceGraph, &bEnforceConstCorrectness))
	{
		Context.MarkAsConstFunction(bEnforceConstCorrectness);
	}

	if (bInternalFunction)
	{
		Context.MarkAsInternalOrCppUseOnly();
	}
}

void FKismetCompilerContext::ValidateFunctionGraphNames()
{
	TSharedPtr<FKismetNameValidator> ParentBPNameValidator;
	if( Blueprint->ParentClass != NULL )
	{
		UBlueprint* ParentBP = Cast<UBlueprint>(Blueprint->ParentClass->ClassGeneratedBy);
		if( ParentBP != NULL )
		{
			ParentBPNameValidator = MakeShareable(new FKismetNameValidator(ParentBP));
		}
	}

	if(ParentBPNameValidator.IsValid())
	{
		TArray<UEdGraph*> AllFunctionGraphs(Blueprint->FunctionGraphs);
		AllFunctionGraphs += GeneratedFunctionGraphs;

		for (UEdGraph* FunctionGraph : AllFunctionGraphs)
		{
			if(FunctionGraph->GetFName() != UEdGraphSchema_K2::FN_UserConstructionScript)
			{
				if( ParentBPNameValidator->IsValid(FunctionGraph->GetName()) != EValidatorResult::Ok )
				{
					FName NewFunctionName = FBlueprintEditorUtils::FindUniqueKismetName(Blueprint, FunctionGraph->GetName());
					MessageLog.Warning(
						*FText::Format(
							LOCTEXT("FunctionGraphConflictWarningFmt", "Found a function graph with a conflicting name ({0}) - changed to {1}."),
							FText::FromString(FunctionGraph->GetName()),
							FText::FromName(NewFunctionName)
						).ToString()
					);
					FBlueprintEditorUtils::RenameGraph(FunctionGraph, NewFunctionName.ToString());
				}
			}
		}
	}
}

// Performs initial validation that the graph is at least well formed enough to be processed further
// Merge separate pages of the ubergraph together into one ubergraph
// Creates a copy of the graph to allow further transformations to occur
void FKismetCompilerContext::CreateFunctionList()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CreateFunctionList);
	{
		BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_GenerateFunctionGraphs);

		// Allow blueprint extensions for the blueprint to generate function graphs
		for (const TObjectPtr<UBlueprintExtension>& Extension : Blueprint->GetExtensions())
		{
			Extension->GenerateFunctionGraphs(this);
		}
	}

	BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_CreateFunctionList);

	// Process the ubergraph if one should be present
	if (FBlueprintEditorUtils::DoesSupportEventGraphs(Blueprint))
	{
		CreateAndProcessUbergraph();
	}

	if (Blueprint->BlueprintType != BPTYPE_MacroLibrary)
	{
		// Ensure that function graph names are valid and that there are no collisions with a parent class
		//ValidateFunctionGraphNames();

		// Run thru the individual function graphs
		for (int32 i = 0; i < Blueprint->FunctionGraphs.Num(); ++i)
		{
			ProcessOneFunctionGraph(Blueprint->FunctionGraphs[i]);
		}

		for (UEdGraph* FunctionGraph : GeneratedFunctionGraphs)
		{
			ProcessOneFunctionGraph(FunctionGraph);
		}

		for (int32 i = 0; i < Blueprint->DelegateSignatureGraphs.Num(); ++i)
		{
			// change function names to unique

			ProcessOneFunctionGraph(Blueprint->DelegateSignatureGraphs[i]);
		}

		// Run through all the implemented interface member functions
		for (int32 i = 0; i < Blueprint->ImplementedInterfaces.Num(); ++i)
		{
			for(int32 j = 0; j < Blueprint->ImplementedInterfaces[i].Graphs.Num(); ++j)
			{
				UEdGraph* SourceGraph = Blueprint->ImplementedInterfaces[i].Graphs[j];
				ProcessOneFunctionGraph(SourceGraph);
			}
		}
	}
}

FKismetFunctionContext* FKismetCompilerContext::CreateFunctionContext()
{
	FKismetFunctionContext* Result = new FKismetFunctionContext(MessageLog, Schema, NewClass, Blueprint);
	FunctionList.Add(Result);
	return Result;
}

/** Compile a blueprint into a class and a set of functions */
void FKismetCompilerContext::CompileClassLayout(EInternalCompilerFlags InternalFlags)
{
	PreCompile();

	// Interfaces only need function signatures, so we only need to perform the first phase of compilation for them
	bIsFullCompile = CompileOptions.DoesRequireBytecodeGeneration() && (Blueprint->BlueprintType != BPTYPE_Interface);

	CallsIntoUbergraph.Empty();
	RepNotifyFunctionMap.Empty();
	if (bIsFullCompile)
	{
		Blueprint->IntermediateGeneratedGraphs.Empty();
	}

	// This flag tries to ensure that component instances will use their template name (since that's how old->new instance mapping is done here)
	//@TODO: This approach will break if and when we multithread compiling, should be an inc-dec pair instead
	TGuardValue<bool> GuardTemplateNameFlag(GCompilingBlueprint, true);

	if (Schema == NULL)
	{
		BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_CreateSchema);
		Schema = CreateSchema();
		PostCreateSchema();
	}

	// Make sure the parent class exists and can be used
	check(Blueprint->ParentClass && Blueprint->ParentClass->GetPropertiesSize());

	UClass* TargetUClass = Blueprint->GeneratedClass;

	// >>> Backwards Compatibility:  Make sure this is an actual UBlueprintGeneratedClass / UAnimBlueprintGeneratedClass, as opposed to the old UClass
	EnsureProperGeneratedClass(TargetUClass);
	// <<< End Backwards Compatibility

	TargetClass = Cast<UBlueprintGeneratedClass>(TargetUClass);

	// >>> Backwards compatibility:  If SkeletonGeneratedClass == GeneratedClass, we need to make a new generated class the first time we need it
	if( Blueprint->SkeletonGeneratedClass == Blueprint->GeneratedClass )
	{
		Blueprint->GeneratedClass = NULL;
		TargetClass = NULL;
	}
	// <<< End Backwards Compatibility

	if( !TargetClass )
	{
		FName NewSkelClassName, NewGenClassName;
		Blueprint->GetBlueprintClassNames(NewGenClassName, NewSkelClassName);
		SpawnNewClass( NewGenClassName.ToString() );
		check(NewClass);

		TargetClass = NewClass;

		// Fix up the reference in the blueprint to the new class
		Blueprint->GeneratedClass = TargetClass;
	}

	// Early validation
	if (CompileOptions.CompileType == EKismetCompileType::Full)
	{
		TArray<UEdGraph*> AllGraphs;
		Blueprint->GetAllGraphs(AllGraphs);
		for (UEdGraph* Graph : AllGraphs)
		{
			if (Graph)
			{
				TArray<UK2Node*> AllNodes;
				Graph->GetNodesOfClass(AllNodes);
				for (UK2Node* Node : AllNodes)
				{
					if (Node)
					{
						Node->EarlyValidation(MessageLog);
					}
				}
			}
		}
	}

	// Ensure that member variable names are valid and that there are no collisions with a parent class
	// This validation requires CDO object.
	ValidateVariableNames();

	if (GetAllowNativeComponentClassOverrides())
	{
		ValidateComponentClassOverrides();
	}

	OldCDO = NULL;
	OldGenLinkerIdx = INDEX_NONE;
	OldLinker = Blueprint->GetLinker();

	if (OldLinker)
	{
		// Cache linker addresses so we can fixup linker for old CDO
		for (int32 i = 0; i < OldLinker->ExportMap.Num(); i++)
		{
			FObjectExport& ThisExport = OldLinker->ExportMap[i];
			if (ThisExport.ObjectFlags & RF_ClassDefaultObject)
			{
				OldGenLinkerIdx = i;
				break;
			}
		}
	}

	for (int32 TimelineIndex = 0; TimelineIndex < Blueprint->Timelines.Num(); )
	{
		if (NULL == Blueprint->Timelines[TimelineIndex])
		{
			Blueprint->Timelines.RemoveAt(TimelineIndex);
			continue;
		}
		++TimelineIndex;
	}


	CleanAndSanitizeClass(TargetClass, OldCDO);

	NewClass->ClassGeneratedBy = Blueprint;
	
	// Set class metadata as needed
	FBlueprintClassFlagUtils::PropagateParentClassFlags_CompileClassLayout(NewClass);
	
	if (FBlueprintEditorUtils::IsInterfaceBlueprint(Blueprint))
	{
		TargetClass->ClassFlags |= CLASS_Interface;
	}

	if(Blueprint->bGenerateConstClass)
	{
		NewClass->ClassFlags |= CLASS_Const;
	}

	if (CompileOptions.CompileType == EKismetCompileType::Full)
	{
		UInheritableComponentHandler* InheritableComponentHandler = Blueprint->GetInheritableComponentHandler(false);
		if (InheritableComponentHandler)
		{
			InheritableComponentHandler->ValidateTemplates();
		}
	}

	IKismetCompilerInterface& KismetCompilerModule = FModuleManager::LoadModuleChecked<IKismetCompilerInterface>("KismetCompiler");
	KismetCompilerModule.ValidateBPAndClassType(Blueprint, MessageLog);

	// If applicable, register any delegate proxy functions and their captured actor variables
	RegisterClassDelegateProxiesFromBlueprint();

	// Run thru the class defined variables first, get them registered
	CreateClassVariablesFromBlueprint();

	// Add any interfaces that the blueprint implements to the class
	// (has to happen before we validate pin links in CreateFunctionList(), so that we can verify self/interface pins)
	AddInterfacesFromBlueprint(NewClass);

	// Construct a context for each function, doing validation and building the function interface
	CreateFunctionList();

	// Function list creation should process captured variables. Something went wrong if we missed any.
	if (!ConvertibleDelegates.IsEmpty())
	{
		UE_LOGF(LogK2Compiler, Warning, "%d convertible delegates were not processed during class layout compilation. Listing delegates in log below.", ConvertibleDelegates.Num());
		for (auto DelegateIt = ConvertibleDelegates.CreateConstIterator(); DelegateIt; ++DelegateIt)
		{
			UE_LOGF(LogK2Compiler, Display, "  Node:%ls Function:%ls Variable:%ls", *GetPathNameSafe(DelegateIt.Key()), *DelegateIt.Value().ProxyFunctionName.ToString(), *DelegateIt.Value().CapturedVariableName.ToString());
		}
	}

	// Precompile the functions
	// Handle delegates signatures first, because they are needed by other functions
	for (int32 i = 0; i < FunctionList.Num(); ++i)
	{
		if(FunctionList[i].IsDelegateSignature())
		{
			PrecompileFunction(FunctionList[i], InternalFlags);
		}
	}

	for (int32 i = 0; i < FunctionList.Num(); ++i)
	{
		if(!FunctionList[i].IsDelegateSignature())
		{
			PrecompileFunction(FunctionList[i], InternalFlags);
		}
	}

	// Now that generated delegate signature functions are compiled & linked to multicast properties, bind generated event nodes
	// The functions have only been compiled to script, so we can initialize the node events now
	InitializeGeneratedEventNodes(InternalFlags);

	if (UsePersistentUberGraphFrame() && UbergraphContext)
	{
		//UBER GRAPH PERSISTENT FRAME
		FEdGraphPinType Type(TEXT("struct"), NAME_None, FPointerToUberGraphFrame::StaticStruct(), EPinContainerType::None, false, FEdGraphTerminalType());
		FProperty* Property = CreateVariable(UBlueprintGeneratedClass::GetUberGraphFrameName(), Type);
		Property->SetPropertyFlags(CPF_DuplicateTransient | CPF_Transient);
	}

	{ BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_BindAndLinkClass);

		// Relink the class
		NewClass->Bind();
		NewClass->StaticLink(true);
	}
}

void FKismetCompilerContext::CompileFunctions(EInternalCompilerFlags InternalFlags)
{
	// This is phase two, so we want to generated locals if PostponeLocalsGenerationUntilPhaseTwo is set:
	const bool bGenerateLocals = !!(InternalFlags & EInternalCompilerFlags::PostponeLocalsGenerationUntilPhaseTwo);
	// Don't propagate values to CDO if we're going to do that in reinstancing:
	const bool bPropagateValuesToCDO = !(InternalFlags & EInternalCompilerFlags::PostponeDefaultObjectAssignmentUntilReinstancing);
	// Don't RefreshExternalBlueprintDependencyNodes if the calling code has done so already:
	const bool bSkipRefreshExternalBlueprintDependencyNodes = !!(InternalFlags & EInternalCompilerFlags::SkipRefreshExternalBlueprintDependencyNodes);
	FKismetCompilerVMBackend Backend_VM(Blueprint, Schema, *this);

	// Validation requires CDO value propagation to occur first.
	bool bSkipGeneratedClassValidation = !bPropagateValuesToCDO;

	if( bGenerateLocals )
	{
		for (int32 i = 0; i < FunctionList.Num(); ++i)
		{
			if (FunctionList[i].IsValid())
			{
				FKismetFunctionContext& Context = FunctionList[i];
				FField::FLinkedListBuilder PropertyListBuilder(&(Context.Function->ChildProperties));
				PropertyListBuilder.MoveToEnd();

				CreateLocalsAndRegisterNets(Context, PropertyListBuilder);
			}
		}
	}

	if (bIsFullCompile && !MessageLog.NumErrors)
	{
		// Generate code for each function (done in a second pass to allow functions to reference each other)
		for (int32 i = 0; i < FunctionList.Num(); ++i)
		{
			if (FunctionList[i].IsValid())
			{
				CompileFunction(FunctionList[i]);
			}
		}

		// Finalize all functions (done last to allow cross-function patchups)
		for (int32 i = 0; i < FunctionList.Num(); ++i)
		{
			if (FunctionList[i].IsValid())
			{
				PostcompileFunction(FunctionList[i]);
			}
		}

		for (TFieldIterator<FMulticastDelegateProperty> PropertyIt(NewClass); PropertyIt; ++PropertyIt)
		{
			if(const FMulticastDelegateProperty* MCDelegateProp = *PropertyIt)
			{
				if(NULL == MCDelegateProp->SignatureFunction)
				{
					MessageLog.Warning(*FString::Printf(TEXT("No SignatureFunction in MulticastDelegateProperty '%s'"), *MCDelegateProp->GetName()));
				}
			}
		}
	}
	else
	{
		// Still need to set flags on the functions even for a skeleton class
		for (int32 i = 0; i < FunctionList.Num(); ++i)
		{
			FKismetFunctionContext& Function = FunctionList[i];
			if (Function.IsValid())
			{
				BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_PostcompileFunction);
				FinishCompilingFunction(Function);
			}
		}
	}

	FunctionListCompiledEvent.Broadcast(this);

	// Save off intermediate build products if requested
	if (bIsFullCompile && CompileOptions.bSaveIntermediateProducts && !Blueprint->bIsRegeneratingOnLoad)
	{
		// Generate code for each function (done in a second pass to allow functions to reference each other)
		for (int32 i = 0; i < FunctionList.Num(); ++i)
		{
			FKismetFunctionContext& ContextFunction = FunctionList[i];
			if (FunctionList[i].SourceGraph != NULL)
			{
				// Record this graph as an intermediate product
				ContextFunction.SourceGraph->Schema = UEdGraphSchema_K2::StaticClass();
				Blueprint->IntermediateGeneratedGraphs.Add(ContextFunction.SourceGraph);
				ContextFunction.SourceGraph->SetFlags(RF_Transient);
			}
		}
	}

	// Late validation for Delegates.
	{
		TSet<UEdGraph*> AllGraphs;
		AllGraphs.Add(UbergraphContext ? UbergraphContext->SourceGraph : NULL);
		for (const FKismetFunctionContext& FunctionContext : FunctionList)
		{
			AllGraphs.Add(FunctionContext.SourceGraph);
		}
		for (UEdGraph* Graph : AllGraphs)
		{
			if (Graph)
			{
				TArray<UK2Node_CreateDelegate*> AllNodes;
				Graph->GetNodesOfClass(AllNodes);
				for (UK2Node_CreateDelegate* Node : AllNodes)
				{
					if (Node)
					{
						Node->ValidationAfterFunctionsAreCreated(MessageLog, (0 != bIsFullCompile));
					}
				}
			}
		}
	}

	{ BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_FinalizationWork);

		// Set any final flags and seal the class, build a CDO, etc...
		FinishCompilingClass(NewClass);

		// Build delegate binding maps if we have a graph
		if (ConsolidatedEventGraph)
		{
			// Build any dynamic binding information for this class
			BuildDynamicBindingObjects(NewClass);
		}

		UObject* NewCDO = NewClass->GetDefaultObject();

		// Copy over the CDO properties if we're not already regenerating on load.  In that case, the copy will be done after compile on load is complete
		if(bPropagateValuesToCDO)
		{
			FBlueprintEditorUtils::PropagateParentBlueprintDefaults(NewClass);

			if( !Blueprint->HasAnyFlags(RF_BeingRegenerated) )
			{
				if( OldCDO )
				{
					// Patch our export table. This also fixes up load flags
					if (OldLinker && OldGenLinkerIdx != INDEX_NONE)
					{
						OldLinker->PRIVATE_PatchNewObjectIntoExport(OldGenLinkerIdx, Blueprint->GeneratedClass->GetDefaultObject());
					}
					// Also patch (or invalidate if no longer existing) subobjects of the class
					FLinkerLoad::PRIVATE_PatchNewObjectIntoExport(TargetClass, TargetClass);

					// Propagate the old CDO's properties to the new
					UEditorEngine::FCopyPropertiesForUnrelatedObjectsParams CopyDetails;
					CopyDetails.bCopyDeprecatedProperties = Blueprint->bIsRegeneratingOnLoad;
					CopyDetails.bNotifyObjectReplacement = true; 
					UEditorEngine::CopyPropertiesForUnrelatedObjects(OldCDO, NewCDO, CopyDetails);
				}
				else
				{
					// Don't perform generated class validation since we didn't do any value propagation.
					bSkipGeneratedClassValidation = true;
				}
			}

			PropagateValuesToCDO(NewCDO, OldCDO);

			// Perform any fixup or caching based on the new CDO.
			PostCDOCompiled(UObject::FPostCDOCompiledContext());
		}

		// Note: The old->new CDO copy is deferred when regenerating, so we skip this step in that case.
		if (!Blueprint->HasAnyFlags(RF_BeingRegenerated))
		{
			// Update the custom property list used in post construction logic to include native class properties for which the Blueprint CDO differs from the native CDO.
			TargetClass->UpdateCustomPropertyListForPostConstruction();
		}
	}

	// Fill out the function bodies, either with function bodies, or simple stubs if this is skeleton generation
	{
		// Should we display debug information about the backend outputs?
		bool bDisplayBytecode = false;

		if (!Blueprint->bIsRegeneratingOnLoad)
		{
			GConfig->GetBool(TEXT("Kismet"), TEXT("CompileDisplaysBinaryBackend"), /*out*/ bDisplayBytecode, GEngineIni);
		}

		// Always run the VM backend, it's needed for more than just debug printing
		{
			const bool bGenerateStubsOnly = !bIsFullCompile || (0 != MessageLog.NumErrors);
			BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_CodeGenerationTime);
			Backend_VM.GenerateCodeFromClass(NewClass, FunctionList, bGenerateStubsOnly);
		}

		// Fill ScriptAndPropertyObjectReferences arrays in functions
		if (bIsFullCompile && (0 == MessageLog.NumErrors)) // Backend_VM can generate errors, so bGenerateStubsOnly cannot be reused
		{
			for (FKismetFunctionContext& FunctionContext : FunctionList)
			{
				if (FunctionContext.IsValid())
				{
					UFunction* Function = FunctionContext.Function;
					auto FunctionScriptAndPropertyObjectReferencesView = MutableView(Function->ScriptAndPropertyObjectReferences);
					FArchiveScriptReferenceCollector ObjRefCollector(FunctionScriptAndPropertyObjectReferencesView, Function);
					for (int32 iCode = 0; iCode < Function->Script.Num();)
					{
						Function->SerializeExpr(iCode, ObjRefCollector);
					}
				}
			}
		}

		if (bDisplayBytecode && bIsFullCompile && !IsRunningCommandlet())
		{
			TGuardValue DisableLogTimes(GPrintLogTimes, ELogTimes::None);

			FKismetBytecodeDisassembler Disasm(*GLog);

			// Disassemble script code
			for (int32 i = 0; i < FunctionList.Num(); ++i)
			{
				FKismetFunctionContext& Function = FunctionList[i];
				if (Function.IsValid())
				{
					UE_LOGF(LogK2Compiler, Log, "\n\n[function %ls]:\n", *(Function.Function->GetName()));
					Disasm.DisassembleStructure(Function.Function);
				}
			}
		}

		static const FBoolConfigValueHelper DisplayLayout(TEXT("Kismet"), TEXT("bDisplaysLayout"), GEngineIni);
		if (!Blueprint->bIsRegeneratingOnLoad && bIsFullCompile && DisplayLayout && NewClass && !IsRunningCommandlet())
		{
			UE_LOGF(LogK2Compiler, Log, "\n\nLAYOUT CLASS %ls:", *GetNameSafe(NewClass));

			for (FProperty* Prop : TFieldRange<FProperty>(NewClass, EFieldIteratorFlags::ExcludeSuper))
			{
				UE_LOGF(LogK2Compiler, Log, "%5d:\t%-64ls\t%ls", Prop->GetOffset_ForGC(), *GetNameSafe(Prop), *Prop->GetCPPType());
			}

			for (UFunction* LocFunction : TFieldRange<UFunction>(NewClass, EFieldIteratorFlags::ExcludeSuper))
			{
				UE_LOGF(LogK2Compiler, Log, "\n\nLAYOUT FUNCTION %ls:", *GetNameSafe(LocFunction));
				for (FProperty* Prop : TFieldRange<FProperty>(LocFunction))
				{
					const bool bOutParam = Prop && (0 != (Prop->PropertyFlags & CPF_OutParm));
					const bool bInParam = Prop && !bOutParam && (0 != (Prop->PropertyFlags & CPF_Parm));
					UE_LOGF(LogK2Compiler, Log, "%5d:\t%-64ls\t%ls %ls%ls"
						, Prop->GetOffset_ForGC(), *GetNameSafe(Prop), *Prop->GetCPPType()
						, bInParam ? TEXT("Input") : TEXT("")
						, bOutParam ? TEXT("Output") : TEXT(""));
				}
			}
		}
	}

	// For full compiles, find other blueprints that may need refreshing, and mark them dirty, in case they try to run
	if( bIsFullCompile && !Blueprint->bIsRegeneratingOnLoad && !bSkipRefreshExternalBlueprintDependencyNodes )
	{
		TArray<UBlueprint*> DependentBlueprints;
		FBlueprintEditorUtils::GetDependentBlueprints(Blueprint, DependentBlueprints);
		for (UBlueprint* CurrentBP : DependentBlueprints)
		{
			// Get the current dirty state of the package
			UPackage* const Package = CurrentBP->GetOutermost();
			const bool bStartedWithUnsavedChanges = Package != nullptr ? Package->IsDirty() : true;
			const EBlueprintStatus OriginalStatus = CurrentBP->Status;

			FBlueprintEditorUtils::RefreshExternalBlueprintDependencyNodes(CurrentBP, NewClass);
			
			// Dependent blueprints will be recompile anyway by reinstancer (if necessary).
			CurrentBP->Status = OriginalStatus;

			// Note: We do not send a change notification event to the dependent BP here because
			// we have not yet reinstanced any of the instances of the BP being compiled, which may
			// be referenced by instances of the dependent BP that may be reconstructed as a result.

			// Clear the package dirty state if it did not initially have any unsaved changes to begin with
			if(Package != nullptr && Package->IsDirty() && !bStartedWithUnsavedChanges)
			{
				Package->SetDirtyFlag(false);
			}
		}
	}

	// Clear out pseudo-local members that are only valid within a Compile call
	UbergraphContext = NULL;
	CallsIntoUbergraph.Empty();
	TimelineToMemberVariableMap.Empty();


	check(NewClass->PropertiesSize >= UObject::StaticClass()->PropertiesSize);
	check(NewClass->GetDefaultObject(false) != NULL);

	PostCompileDiagnostics();

	// Perform validation only if CDO propagation was performed above, otherwise the new CDO will not yet be fully initialized.
	if (bIsFullCompile && !bSkipGeneratedClassValidation && !Blueprint->bIsRegeneratingOnLoad)
	{
		bool Result = ValidateGeneratedClass(NewClass);
		// TODO What do we do if validation fails?
	}

	if (bIsFullCompile)
	{
		BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_ChecksumCDO);

		static const FBoolConfigValueHelper ChangeDefaultValueWithoutReinstancing(TEXT("Kismet"), TEXT("bChangeDefaultValueWithoutReinstancing"), GEngineIni);
		// CRC is usually calculated for all Properties. If the bChangeDefaultValueWithoutReinstancing optimization is enabled, then only specific properties are considered (in fact we should consider only . See UE-9883.
		// Some native properties (bCanEverTick) may be implicitly changed by KismetCompiler during compilation, so they always need to be compared.
		// Some properties with a custom Property Editor Widget may not propagate changes among instances. They may be also compared.

		class FSpecializedArchiveCrc32 : public FArchiveObjectCrc32
		{
		public:
			bool bAllProperties;

			FSpecializedArchiveCrc32(bool bInAllProperties)
				: FArchiveObjectCrc32()
				, bAllProperties(bInAllProperties)
			{}

			static bool PropertyCanBeImplicitlyChanged(const FProperty* InProperty)
			{
				check(InProperty);

				UClass* PropertyOwnerClass = InProperty->GetOwnerClass();
				const bool bOwnerIsNativeClass = PropertyOwnerClass && PropertyOwnerClass->HasAnyClassFlags(CLASS_Native);

				UStruct* PropertyOwnerStruct = InProperty->GetOwnerStruct();
				const bool bOwnerIsNativeStruct = !PropertyOwnerClass && (!PropertyOwnerStruct || !PropertyOwnerStruct->IsA<UUserDefinedStruct>());

				return InProperty->IsA<FStructProperty>()
					|| bOwnerIsNativeClass || bOwnerIsNativeStruct;
			}

			// Begin FArchive Interface
			virtual bool ShouldSkipProperty(const FProperty* InProperty) const override
			{
				return FArchiveObjectCrc32::ShouldSkipProperty(InProperty) 
					|| (!bAllProperties && !PropertyCanBeImplicitlyChanged(InProperty));
			}
			// End FArchive Interface
		};

		UObject* NewCDO = NewClass->GetDefaultObject(false);
		FSpecializedArchiveCrc32 CrcArchive(!ChangeDefaultValueWithoutReinstancing);
		Blueprint->CrcLastCompiledCDO = NewCDO ? CrcArchive.Crc32(NewCDO) : 0;
	}

	if (bIsFullCompile)
	{
		BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_ChecksumSignature);

		class FSignatureArchiveCrc32 : public FArchiveObjectCrc32
		{
		public:

			static bool IsInnerProperty(const FField* Field)
			{
				const FProperty* Property = CastField<const FProperty>(Field);
				return Property // check arrays
					&& Cast<const UFunction>(Property->GetOwnerStruct())
					&& !Property->HasAnyPropertyFlags(CPF_Parm);
			}

			virtual FArchive& operator<<(FField*& Field) override
			{
				FArchive& Ar = *this;
				if (Field && !IsInnerProperty(Field))
				{					
					FString UniqueName = GetPathNameSafe(Field);
					Ar << UniqueName;
					if (Field->IsIn(RootObject))
					{
						Field->Serialize(Ar);
					}
				}
				return Ar;
			}

			virtual FArchive& operator<<(UObject*& Object) override
			{
				FArchive& Ar = *this;

				if (Object)
				{
					// Names of functions and properties are significant.
					FString UniqueName = GetPathNameSafe(Object);
					Ar << UniqueName;

					if (Object->IsIn(RootObject))
					{
						ObjectsToSerialize.Enqueue(Object);
					}
				}

				return Ar;
			}

			virtual bool CustomSerialize(UObject* Object) override
			{ 
				FArchive& Ar = *this;

				bool bResult = false;
				if (UStruct* Struct = Cast<UStruct>(Object))
				{
					if (Object == RootObject) // name and location are significant for the signature
					{
						FString UniqueName = GetPathNameSafe(Object);
						Ar << UniqueName;
					}

					UObject* SuperStruct = Struct->GetSuperStruct();
					Ar << SuperStruct;

					UField* ChildrenIter = Struct->Children;
					while(ChildrenIter)
					{
						Ar << ChildrenIter;
						ChildrenIter = ChildrenIter->Next;
					}

					FField* ChildPropIter = Struct->ChildProperties;
					while (ChildPropIter)
					{
						Ar << ChildPropIter;
						ChildPropIter = ChildPropIter->Next;
					}

					if (UFunction* Function = Cast<UFunction>(Struct))
					{
						Ar << Function->FunctionFlags;
					}

					if (UClass* AsClass = Cast<UClass>(Struct))
					{
						Ar << (uint32&)AsClass->ClassFlags;
						Ar << AsClass->Interfaces;
					}

					Ar << Struct->Next;

					bResult = true;
				}

				return bResult;
			}
		};

		FSignatureArchiveCrc32 SignatureArchiveCrc32;
		UBlueprint* ParentBP = UBlueprint::GetBlueprintFromClass(NewClass->GetSuperClass());
		const uint32 ParentSignatureCrc = ParentBP ? ParentBP->CrcLastCompiledSignature : 0;
		Blueprint->CrcLastCompiledSignature = SignatureArchiveCrc32.Crc32(NewClass, ParentSignatureCrc);
	}

	PostCompile();
}

void FKismetCompilerContext::PostCDOCompiled(const UObject::FPostCDOCompiledContext& Context)
{
	// Exit now if we did not create a new class.
	if (!NewClass)
	{
		return;
	}

	// Conform any sparse class data now that the CDO is ready
	NewClass->ConformSparseClassData(NewClass->GetDefaultObject(false));

	// Notify the CDO that it has finished compiling
	NewClass->GetDefaultObject(false)->PostCDOCompiled(Context);
	if (!Context.bIsSkeletonOnly)
	{
		for (const TFunction<void(const UObject::FPostCDOCompiledContext&, UObject*)>& Step : PostCDOCompileSteps)
		{
			Step(Context, NewClass->GetDefaultObject(false));
		}
	}
	
	FCoreUObjectDelegates::OnObjectPostCDOCompiled.Broadcast(NewClass->GetDefaultObject(false), Context);

	// Allow children to customize PostCDOCompile:
	OnPostCDOCompiled(Context);

	// Run this function in case any native default property values have been changed on the CDO
	// from all the user code above
	NewClass->UpdateCustomPropertyListForPostConstruction();
}

void FKismetCompilerContext::AddPostCDOCompiledStep(TFunction<void(const UObject::FPostCDOCompiledContext&, UObject*)>&& StepFunction)
{
	PostCDOCompileSteps.Emplace(StepFunction);
}

void FKismetCompilerContext::SetAutomaticallyRegisterInputEvents(const bool bVal)
{
	bAutomaticallyRegisterInputEvents = bVal;
}

bool FKismetCompilerContext::GetAutomaticallyRegisterInputEvents() const
{
	return bAutomaticallyRegisterInputEvents;
}

void FKismetCompilerContext::Compile()
{
	CompileClassLayout(EInternalCompilerFlags::None);
	CompileFunctions(EInternalCompilerFlags::None);
}

void FKismetCompilerContext::SetNewClass(UBlueprintGeneratedClass* ClassToUse)
{
	NewClass = ClassToUse;
	OnNewClassSet(ClassToUse);
}

bool FKismetCompilerContext::ValidateGeneratedClass(UBlueprintGeneratedClass* Class)
{
	return UBlueprint::ValidateGeneratedClass(Class);
}

UEdGraph* FKismetCompilerContext::SpawnIntermediateFunctionGraph(const FString& InDesiredFunctionName, const UFunction* InSignature, bool bUseUniqueName)
{
	FName UniqueGraphName = bUseUniqueName ? FBlueprintEditorUtils::FindUniqueKismetName(Blueprint, InDesiredFunctionName) : *InDesiredFunctionName;

	UEdGraph* GeneratedFunctionGraph = FBlueprintEditorUtils::CreateNewGraph(Blueprint, UniqueGraphName, UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());
	GeneratedFunctionGraph->SetFlags(RF_Transient);
	GeneratedFunctionGraph->bEditable = false;

	FBlueprintEditorUtils::CreateFunctionGraph(Blueprint, GeneratedFunctionGraph, false, InSignature);

	// Add the function graph to the list of generated graphs for this compile
	GeneratedFunctionGraphs.Add(GeneratedFunctionGraph);
	return GeneratedFunctionGraph;
}

const UK2Node_FunctionEntry* FKismetCompilerContext::FindLocalEntryPoint(const UFunction* Function) const
{
	for (int32 i = 0; i < FunctionList.Num(); ++i)
	{
		const FKismetFunctionContext& FunctionContext = FunctionList[i];
		if (FunctionContext.IsValid() && FunctionContext.Function == Function)
		{
			return FunctionContext.EntryPoint;
		}
	}
	return NULL;
}

void FKismetCompilerContext::SetCanEverTick() const
{
	FTickFunction* TickFunction = nullptr;
	FTickFunction* ParentTickFunction = nullptr;
	
	if (AActor* CDActor = Cast<AActor>(NewClass->GetDefaultObject()))
	{
		TickFunction = &CDActor->PrimaryActorTick;
		ParentTickFunction = &NewClass->GetSuperClass()->GetDefaultObject<AActor>()->PrimaryActorTick;
	}
	else if (UActorComponent* CDComponent = Cast<UActorComponent>(NewClass->GetDefaultObject()))
	{
		TickFunction = &CDComponent->PrimaryComponentTick;
		ParentTickFunction = &NewClass->GetSuperClass()->GetDefaultObject<UActorComponent>()->PrimaryComponentTick;
	}

	if (TickFunction == nullptr)
	{
		return;
	}

	const bool bOldFlag = TickFunction->bCanEverTick;
	// RESET FLAG 
	TickFunction->bCanEverTick = ParentTickFunction->bCanEverTick;
	
	// RECEIVE TICK
	if (!TickFunction->bCanEverTick)
	{
		// Make sure that both AActor and UActorComponent have the same name for their tick method
		static FName ReceiveTickName(GET_FUNCTION_NAME_CHECKED(AActor, ReceiveTick));
		static FName ComponentReceiveTickName(GET_FUNCTION_NAME_CHECKED(UActorComponent, ReceiveTick));

		if (const UFunction* ReceiveTickEvent = FKismetCompilerUtilities::FindOverriddenImplementableEvent(ReceiveTickName, NewClass))
		{
			// We have a tick node, but are we allowed to?

			const UEngine* EngineSettings = GetDefault<UEngine>();
			const bool bAllowTickingByDefault = EngineSettings->bCanBlueprintsTickByDefault;

			const UClass* FirstNativeClass = FBlueprintEditorUtils::FindFirstNativeClass(NewClass);
			const bool bHasCanTickMetadata = (FirstNativeClass != nullptr) && FirstNativeClass->HasMetaData(FBlueprintMetadata::MD_ChildCanTick);
			const bool bHasCannotTickMetadata = (FirstNativeClass != nullptr) && FirstNativeClass->HasMetaData(FBlueprintMetadata::MD_ChildCannotTick);
			const bool bHasUniversalParent = (FirstNativeClass != nullptr) && ((AActor::StaticClass() == FirstNativeClass) || (UActorComponent::StaticClass() == FirstNativeClass));

			if (bHasCanTickMetadata && bHasCannotTickMetadata)
			{
				// User error: The C++ class has conflicting metadata
				const FString ConlictingMetadataWarning = FText::Format(
					LOCTEXT("HasBothCanAndCannotMetadataFmt", "Native class {0} has both '{1}' and '{2}' metadata specified, they are mutually exclusive and '{2}' will win."),
					FText::FromString(FirstNativeClass->GetPathName()),
					FText::FromName(FBlueprintMetadata::MD_ChildCanTick),
					FText::FromName(FBlueprintMetadata::MD_ChildCannotTick)
				).ToString();
				MessageLog.Warning(*ConlictingMetadataWarning);
			}

			// If the tick node is disconnected we can avoid ticking needlessly
			bool bIsEmptyBlueprintTick = false;
			for (const FKismetFunctionContext& FunctionContext : FunctionList)
			{
				if (FunctionContext.IsValid() && FunctionContext.Function->GetName() == ReceiveTickName)
				{
					if (FunctionContext.SourceEventFromStubGraph)
					{
						UEdGraphPin* ExecThenPin = FunctionContext.SourceEventFromStubGraph->FindPin(UEdGraphSchema_K2::PN_Then);
						bIsEmptyBlueprintTick = ExecThenPin && ExecThenPin->LinkedTo.IsEmpty();
					}
					break;
				}
			}

			if (bHasCannotTickMetadata)
			{
				// This could only happen if someone adds bad metadata to AActor or UActorComponent directly
				check(!bHasUniversalParent);

				// Parent class has forbidden us to tick
				const FString NativeClassSaidNo = FText::Format(
					LOCTEXT("NativeClassProhibitsTickingFmt", "@@ is not allowed as the C++ parent class {0} has disallowed Blueprint subclasses from ticking.  Please consider using a Timer instead of Tick."),
					FText::FromString(FirstNativeClass->GetPathName())
				).ToString();
				MessageLog.Warning(*NativeClassSaidNo, FindLocalEntryPoint(ReceiveTickEvent));
			}
			else if (!bIsEmptyBlueprintTick)
			{
				if (bAllowTickingByDefault || bHasUniversalParent || bHasCanTickMetadata)
				{
					// We're allowed to tick for one reason or another
					TickFunction->bCanEverTick = true;
				}
				else
				{
					// Nothing allowing us to tick
					const FString ReceiveTickEventWarning = FText::Format(
						LOCTEXT("ReceiveTick_CanNeverTickFmt", "@@ is not allowed for Blueprints based on the C++ parent class {0}, so it will never Tick!"),
						FText::FromString(FirstNativeClass ? *FirstNativeClass->GetPathName() : TEXT("<null>"))
					).ToString();
					MessageLog.Warning(*ReceiveTickEventWarning, FindLocalEntryPoint(ReceiveTickEvent));

					const FString ReceiveTickEventRemedies = FText::Format(
						LOCTEXT("ReceiveTick_CanNeverTickRemediesFmt", "You can solve this in several ways:\n  1) Consider using a Timer instead of Tick.\n  2) Add meta=({0}) to the parent C++ class\n  3) Reparent the Blueprint to AActor or UActorComponent, which can always tick."),
						FText::FromName(FBlueprintMetadata::MD_ChildCanTick)
					).ToString();
					MessageLog.Warning(*ReceiveTickEventRemedies);
				}
			}
		}
	}

	if (TickFunction->bCanEverTick != bOldFlag)
	{
		const FCoreTexts& CoreTexts = FCoreTexts::Get();

		UE_LOGF(LogK2Compiler, Verbose, "Overridden flag for class '%ls': CanEverTick %ls ", *NewClass->GetName(),
			TickFunction->bCanEverTick ? *(CoreTexts.True.ToString()) : *(CoreTexts.False.ToString()) );
	}
}

bool FKismetCompilerContext::UsePersistentUberGraphFrame() const
{
	return UBlueprintGeneratedClass::UsePersistentUberGraphFrame();
}

FString FKismetCompilerContext::GetGuid(const UEdGraphNode* Node) const
{
	// We need a unique, deterministic name for the properties we're generating, but the chance of collision is small
	// so I think we can get away with stomping part of a guid with a hash:
	uint32 ResultCRC = FCrc::MemCrc32(&(Node->NodeGuid), sizeof(Node->NodeGuid));
	UEdGraphNode* const* SourceNode = MacroGeneratedNodes.Find(Node);
	while (SourceNode && *SourceNode)
	{
		ResultCRC = FCrc::MemCrc32(&((*SourceNode)->NodeGuid), sizeof((*SourceNode)->NodeGuid), ResultCRC);
		SourceNode = MacroGeneratedNodes.Find(*SourceNode);
	}

	FGuid Ret = Node->NodeGuid;
	Ret.D = ResultCRC;
	return Ret.ToString();
}

TMap< UClass*, CompilerContextFactoryFunction> CustomCompilerMap;

TSharedPtr<FKismetCompilerContext> FKismetCompilerContext::GetCompilerForBP(UBlueprint* BP, FCompilerResultsLog& InMessageLog, const FKismetCompilerOptions& InCompileOptions)
{
	if(CompilerContextFactoryFunction* FactoryFunction = CustomCompilerMap.Find(BP->GetClass()))
	{
		return (*FactoryFunction)(BP, InMessageLog, InCompileOptions);
	}
	else
	{
		return TSharedPtr<FKismetCompilerContext>(new FKismetCompilerContext(BP, InMessageLog, InCompileOptions));
	}
}

void FKismetCompilerContext::RegisterCompilerForBP(UClass* BPClass, CompilerContextFactoryFunction FactoryFunction)
{
	CustomCompilerMap.Add(BPClass, FactoryFunction);
}

void FKismetCompilerContext::MapExpansionPathToTunnelInstance(const UEdGraphNode* InnerExpansionNode, const UEdGraphNode* OuterTunnelInstance)
{
	if (InnerExpansionNode && OuterTunnelInstance)
	{
		// Only map the node to the tunnel instance if it hasn't been mapped before (e.g. by a nested expansion).
		if (!MessageLog.GetIntermediateTunnelInstance(InnerExpansionNode))
		{
			MessageLog.NotifyIntermediateTunnelNode(InnerExpansionNode, OuterTunnelInstance);
		}

		// Recursively map any nodes linked to this node along each output execution path.
		for (const UEdGraphPin* OutputPin : InnerExpansionNode->Pins)
		{
			if (OutputPin->Direction == EGPD_Output && UEdGraphSchema_K2::IsExecPin(*OutputPin) && OutputPin->LinkedTo.Num() > 0)
			{
				for (const UEdGraphPin* LinkedTo : OutputPin->LinkedTo)
				{
					// Make sure it is valid and hasn't already been mapped (e.g. shared execution paths). Also, avoid mapping tunnel output nodes (not needed).
					const UEdGraphNode* LinkedExpansionNode = LinkedTo->GetOwningNode();
					if (LinkedExpansionNode
						&& !MessageLog.GetIntermediateTunnelInstance(LinkedExpansionNode)
						&& (!LinkedExpansionNode->IsA<UK2Node_Tunnel>() || FBlueprintEditorUtils::IsTunnelInstanceNode(LinkedExpansionNode)))
					{
						MapExpansionPathToTunnelInstance(LinkedExpansionNode, OuterTunnelInstance);
					}
				}
			}
		}
	}
}

// This method injects an intermediate "boundary" node on either side of a tunnel instance node and the tunnel input/output
// nodes which can be found along the execution path that flows through the tunnel instance node's expansion. The boundary
// nodes resolve to a NOP debug site for breakpoints and wire traces, and are only constructed when debug data is enabled.
//
// For example:
//
//     +======================+
//     | Tunnel instance node |
//     +======================+
// (1) | >--+            +--> |
//     +====|============|====+
//          |            |
//          |            +-------------------------------------------------------+
//          |                                                                    |
//          |   +================+                         +=================+   |
//          |   | Input (Tunnel) |                         | Output (Tunnel) |   |
//          |   +================+                         +=================+   |
//          +---|--------------> | (2) . . . . . . . . (3) | >---------------|---+
//              +================+                         +=================+
//
// In the expansion shown above, intermediate boundary nodes are created at the following locations along the execution path:
//
//	(1) "Entry" site - Precedes the tunnel instance node in the execution sequence.
//	(2) "Input" site - Follows the input tunnel in the expansion of the tunnel instance.
//	(3) "Output" site - Precedes the output tunnel in the expansion of the tunnel instance.
//
// After tunnels are collapsed and isolated in the intermediate function graph during expansion, the tunnel boundary nodes will
// remain in place along the execution path, and they won't get compiled out. The resulting bytecode resolves to a NOP sequence.
//
// When a tunnel instance node has multiple exec inputs/outputs, this method creates one tunnel boundary per exec path through
// the expansion. Also, note that we do not create a boundary node on the output side of the tunnel instance node, because we want
// execution to continue on to the next linked node after the instruction pointer passes the tunnel output site when single-stepping.
//
// In addition to creating intermediate tunnel boundary nodes, this method also maps the intermediate impure nodes along each unique
// execution path through the expansion (between boundaries 2 and 3 in the diagram above) back to the intermediate tunnel instance
// node that resulted in the expansion. This mapping is used for (a) producing stable UUIDs for latent nodes in an expansion, and
// (b) drawing "marching ants" on either side of the tunnel instance node that corresponds to the execution path in the source graph. 
//
void FKismetCompilerContext::ProcessIntermediateTunnelBoundary(UK2Node_Tunnel* TunnelInput, UK2Node_Tunnel* TunnelOutput)
{
	// @TODO move this check out of KismetFunctionContext so we can use it here?
	auto IsDebuggingOrInstrumentationRequired = []() -> bool
	{
		return GIsEditor && !IsRunningCommandlet();
	};

	// Common initialization.
	auto InitializeTunnelBoundaryNode = [this](UK2Node_TunnelBoundary* TunnelBoundary, UK2Node_Tunnel* TunnelSource)
	{
		// Set the base node name and boundary type.
		TunnelBoundary->SetNodeAttributes(TunnelSource);

		// Position the node in the intermediate graph.
		TunnelBoundary->NodePosX = TunnelSource->NodePosX;
		TunnelBoundary->NodePosY = TunnelSource->NodePosY;
	};

	if (TunnelInput)
	{
		// Flag that indicates whether or not the tunnel instance node is designated as an input or an output.
		const bool bIsTunnelEntrySite = FBlueprintEditorUtils::IsTunnelInstanceNode(TunnelInput);

		for (UEdGraphPin* InputPin : TunnelInput->Pins)
		{
			// We create a boundary node for each exec pin input. This way every execution path has a debug site.
			if (InputPin->Direction == EGPD_Input && UEdGraphSchema_K2::IsExecPin(*InputPin) && InputPin->LinkedTo.Num() > 0)
			{
				if (IsDebuggingOrInstrumentationRequired())
				{
					// Create one or more boundary nodes that precede the tunnel input node.
					if (UK2Node_TunnelBoundary* InputBoundaryNode = SpawnIntermediateNode<UK2Node_TunnelBoundary>(TunnelInput))
					{
						InitializeTunnelBoundaryNode(InputBoundaryNode, TunnelInput);

						// Map the intermediate input tunnel boundary node back to the intermediate tunnel instance node that spawned it.
						MessageLog.NotifyIntermediateTunnelNode(InputBoundaryNode, bIsTunnelEntrySite ? TunnelInput : TunnelOutput);

						if (UEdGraphPin* NewInputPin = InputBoundaryNode->CreatePin(EGPD_Input, InputPin->PinType, InputPin->PinName))
						{
							if (UEdGraphPin* NewOutputPin = InputBoundaryNode->CreatePin(EGPD_Output, InputPin->PinType, InputBoundaryNode->CreateUniquePinName(InputPin->PinName)))
							{
								// Move the exec pin links to the boundary node. This ensures that execution will flow through the boundary node.
								if (MovePinLinksToIntermediate(*InputPin, *NewInputPin).CanSafeConnect())
								{
									NewOutputPin->MakeLinkTo(InputPin);
								}
							}
						}
					}
				}

				// Look for a matching pin on the tunnel output node.
				if (UEdGraphPin* OutputPin = TunnelOutput ? TunnelOutput->FindPin(InputPin->PinName) : nullptr)
				{
					if (ensure(OutputPin->Direction == EGPD_Output && UEdGraphSchema_K2::IsExecPin(*OutputPin)) && OutputPin->LinkedTo.Num() > 0)
					{
						if (bIsTunnelEntrySite)
						{
							// Map the execution path through the expansion back to the tunnel instance node. Note that the assumption here is
							// that we haven't collapsed the tunnels yet, so the output side of the expansion shouldn't be linked to anything.
							for (UEdGraphPin* LinkedTo : OutputPin->LinkedTo)
							{
								MapExpansionPathToTunnelInstance(LinkedTo->GetOwningNode(), TunnelInput);
							}

							if (IsDebuggingOrInstrumentationRequired())
							{
								// We also create a boundary node for each matching exec pin on the tunnel output node.
								if (UK2Node_TunnelBoundary* OutputBoundaryNode = SpawnIntermediateNode<UK2Node_TunnelBoundary>(TunnelOutput))
								{
									InitializeTunnelBoundaryNode(OutputBoundaryNode, TunnelOutput);

									// Map the intermediate output tunnel boundary node back to the intermediate tunnel instance node that spawned it.
									MessageLog.NotifyIntermediateTunnelNode(OutputBoundaryNode, TunnelInput);

									if (UEdGraphPin* NewInputPin = OutputBoundaryNode->CreatePin(EGPD_Input, OutputPin->PinType, OutputPin->PinName))
									{
										if (UEdGraphPin* NewOutputPin = OutputBoundaryNode->CreatePin(EGPD_Output, OutputPin->PinType, OutputBoundaryNode->CreateUniquePinName(OutputPin->PinName)))
										{
											// Move the exec pin links to the boundary node. This ensures that execution will flow through the boundary node.
											if (MovePinLinksToIntermediate(*OutputPin, *NewOutputPin).CanSafeConnect())
											{
												NewInputPin->MakeLinkTo(OutputPin);
											}
										}
									}
								}
							}
						}
						else if(IsDebuggingOrInstrumentationRequired())
						{
							// This is the output side of the expansion, so a tunnel boundary node will not be required on the output side of the pair. However, for
							// wire traces to function properly, we still need to map exec pins linked to the input side back to the matching pin on the output side.
							for (UEdGraphPin* LinkedInputPin : InputPin->LinkedTo)
							{
								MessageLog.NotifyIntermediatePinCreation(LinkedInputPin, OutputPin);
							}
						}
					}
				}
			}
		}
	}
}

void FKismetCompilerContext::RegisterConvertibleDelegates(UEdGraph* Graph)
{
	check(Graph);

	TArray<UK2Node_CreateDelegate*> CreateDelegateNodes;
	Graph->GetNodesOfClass<UK2Node_CreateDelegate>(CreateDelegateNodes);

	for (UK2Node_CreateDelegate* CreateDelegateNode : CreateDelegateNodes)
	{
		check(CreateDelegateNode);

		FMemberReference MemberReference;
		MemberReference.SetDirect(CreateDelegateNode->SelectedFunctionName, CreateDelegateNode->SelectedFunctionGuid, CreateDelegateNode->GetScopeClass(), false);

		const UFunction* BoundFunction = MemberReference.ResolveMember<UFunction>();
		const UFunction* DelegateSignature = CreateDelegateNode->GetDelegateSignature();

		if (BoundFunction && DelegateSignature)
		{
			ConvertibleSignatureMatchResult Result = FKismetCompilerUtilities::DoSignaturesHaveConvertibleFloatTypes(BoundFunction, DelegateSignature);
			if (Result == ConvertibleSignatureMatchResult::HasConvertibleFloatParams)
			{
				int Counter = ConvertibleDelegates.Num();

				FDelegateInfo DelegateInfo;
				DelegateInfo.ProxyFunctionName = *FString::Printf(TEXT("CREATEDELEGATE_PROXYFUNCTION_%d"), Counter);

				const UEdGraphPin* DelegateNodeSelfPin = CreateDelegateNode->FindPinChecked(UEdGraphSchema_K2::PSC_Self);
				if (DelegateNodeSelfPin->LinkedTo.Num() > 0)
				{
					const UEdGraphPin* TargetActorPin = DelegateNodeSelfPin->LinkedTo[0];
					check(TargetActorPin);

					DelegateInfo.CapturedVariableName = *FString::Printf(TEXT("CREATEDELEGATE_CAPTUREDVARIABLE_%d"), Counter);

					verify(CreateVariable(DelegateInfo.CapturedVariableName, TargetActorPin->PinType) != nullptr);
				}

				ConvertibleDelegates.Add(CreateDelegateNode, MoveTemp(DelegateInfo));
			}
		}
	}
}

void FKismetCompilerContext::ReplaceConvertibleDelegates(UEdGraph* Graph)
{
	check(Graph);

	TArray<UK2Node_CreateDelegate*> CreateDelegateNodes;
	Graph->GetNodesOfClass<UK2Node_CreateDelegate>(CreateDelegateNodes);

	for (UK2Node_CreateDelegate* CreateDelegateNode : CreateDelegateNodes)
	{
		check(CreateDelegateNode);

		const UK2Node_CreateDelegate* SourceCreateDelegateNode = Cast<const UK2Node_CreateDelegate>(MessageLog.FindSourceObject(CreateDelegateNode));

		if (const FDelegateInfo* DelegateInfo = ConvertibleDelegates.Find(SourceCreateDelegateNode))
		{
			// Create the new function graph

			const UFunction* DelegateSignature = CreateDelegateNode->GetDelegateSignature();
			UEdGraph* DelegateProxyGraph = SpawnIntermediateFunctionGraph(DelegateInfo->ProxyFunctionName.ToString(), DelegateSignature, false);
			check(DelegateProxyGraph);
			const UEdGraphSchema_K2* K2Schema = CastChecked<UEdGraphSchema_K2>(Graph->GetSchema());

			// Create a function call node from the original delegate signature

			FMemberReference MemberReference;
			MemberReference.SetDirect(CreateDelegateNode->SelectedFunctionName, CreateDelegateNode->SelectedFunctionGuid, CreateDelegateNode->GetScopeClass(), false);
			const UFunction* BoundFunction = MemberReference.ResolveMember<UFunction>();
			UK2Node_CallFunction* BoundFunctionNode = SpawnIntermediateNode<UK2Node_CallFunction>(CreateDelegateNode, DelegateProxyGraph);
			check(BoundFunctionNode);
			BoundFunctionNode->SetFromFunction(BoundFunction);
			BoundFunctionNode->AllocateDefaultPins();

			// Link the entry node and the function node pins

			TArray<UK2Node_FunctionEntry*> EntryNodes;
			DelegateProxyGraph->GetNodesOfClass<UK2Node_FunctionEntry>(EntryNodes);
			check(EntryNodes.Num() == 1);
			UK2Node_FunctionEntry* ProxyEntryNode = EntryNodes[0];
			check(ProxyEntryNode);
			UEdGraphPin* EntryNodeThenPin = ProxyEntryNode->FindPinChecked(UEdGraphSchema_K2::PN_Then);
			UEdGraphPin* FunctionNodeExecPin = BoundFunctionNode->GetExecPin();
			verify(K2Schema->TryCreateConnection(EntryNodeThenPin, FunctionNodeExecPin));

			// Link function entry outputs to the bound function's inputs.
			// Note that we intentionally skip the "self" pin, since that's a special condition that we handle separately.

			int EntryNodePinCursor = -1;
			int FunctionNodePinCursor = -1;

			for (int i = 0; i < ProxyEntryNode->Pins.Num(); ++i)
			{
				const UEdGraphPin* CurrentPin = ProxyEntryNode->Pins[i];
				check(CurrentPin);
				if (!K2Schema->IsMetaPin(*CurrentPin))
				{
					EntryNodePinCursor = i;
					break;
				}
			}
			check(EntryNodePinCursor != -1);

			for (int i = 0; i < BoundFunctionNode->Pins.Num(); ++i)
			{
				const UEdGraphPin* CurrentPin = BoundFunctionNode->Pins[i];
				check(CurrentPin)
				if (!K2Schema->IsMetaPin(*CurrentPin) && !K2Schema->IsSelfPin(*CurrentPin))
				{
					FunctionNodePinCursor = i;
					break;
				}
			}
			check(FunctionNodePinCursor != -1);

			while (EntryNodePinCursor < ProxyEntryNode->Pins.Num())
			{
				UEdGraphPin* OutputPin = ProxyEntryNode->Pins[EntryNodePinCursor];
				UEdGraphPin* InputPin = BoundFunctionNode->Pins[FunctionNodePinCursor];

				verify(K2Schema->TryCreateConnection(OutputPin, InputPin));

				++EntryNodePinCursor;
				++FunctionNodePinCursor;
			}

			// Add and set a capture variable if our self pin is linked to a different actor.

			UEdGraphPin* DelegateNodeSelfPin = CreateDelegateNode->FindPinChecked(UEdGraphSchema_K2::PSC_Self);
			if (DelegateNodeSelfPin->LinkedTo.Num() > 0)
			{
				check(DelegateNodeSelfPin->LinkedTo.Num() == 1);

				// Modify main graph
				{
					// Break our delegate node's current self pin
					// Our replacement proxy func implies the use of 'self'

					UEdGraphPin* TargetActorPin = DelegateNodeSelfPin->LinkedTo[0];
					check(TargetActorPin);
					DelegateNodeSelfPin->BreakAllPinLinks();

					// Create our setter node for the captured variable

					UK2Node_VariableSet* SetCapturedVarNode = SpawnIntermediateNode<UK2Node_VariableSet>(CreateDelegateNode, Graph);
					check(SetCapturedVarNode);
					SetCapturedVarNode->VariableReference.SetSelfMember(DelegateInfo->CapturedVariableName);
					SetCapturedVarNode->AllocateDefaultPins();

					// Link the setter node's input to the original actor pin

					UEdGraphPin* SetCapturedVarNodeVariablePin = SetCapturedVarNode->FindPinChecked(DelegateInfo->CapturedVariableName, EGPD_Input);
					verify(K2Schema->TryCreateConnection(TargetActorPin, SetCapturedVarNodeVariablePin));

					// In order to set the capture variable, we need to find an execution path to insert our setter node.
					// The best likely candidate is the linked UK2Node_BaseMCDelegate node, which either binds or unbinds a delegate.
					// We can insert our setter node to its execution pin.

					UEdGraphPin* DelegateOutPin = CreateDelegateNode->GetDelegateOutPin();
					check(DelegateOutPin);
					check(DelegateOutPin->LinkedTo.Num() == 1);

					UEdGraphPin* TargetDelegatePin = FBlueprintEditorUtils::FindFirstCompilerRelevantLinkedPin(DelegateOutPin->LinkedTo[0]);
					check(TargetDelegatePin);
					UK2Node_BaseMCDelegate* BaseMCDelegateNode = CastChecked<UK2Node_BaseMCDelegate>(TargetDelegatePin->GetOwningNode());

					// Insert the setter node

					UEdGraphPin* SetCapturedVarNodeExecPin = SetCapturedVarNode->GetExecPin();
					UEdGraphPin* SetCapturedVarNodeThenPin = SetCapturedVarNode->GetThenPin();
					UEdGraphPin* BaseMCDelegateNodeExecPin = BaseMCDelegateNode->GetExecPin();

					for (UEdGraphPin* LinkedPin : BaseMCDelegateNodeExecPin->LinkedTo)
					{
						check(LinkedPin);
						UEdGraphPin* InputThenPin = FBlueprintEditorUtils::FindFirstCompilerRelevantLinkedPin(LinkedPin);
						BaseMCDelegateNodeExecPin->BreakLinkTo(InputThenPin);

						verify(K2Schema->TryCreateConnection(InputThenPin, SetCapturedVarNodeExecPin));
						verify(K2Schema->TryCreateConnection(SetCapturedVarNodeThenPin, BaseMCDelegateNodeExecPin));
					}

					// Node expansion for the setter node needs to happen, which prunes the default 'get' pin that's normally added during pin allocation

					SetCapturedVarNode->ExpandNode(*this, Graph);
				}

				// Modify proxy delegate graph
				{
					// Create our getter node for the captured variable

					UK2Node_VariableGet* GetCapturedVarNode = SpawnIntermediateNode<UK2Node_VariableGet>(CreateDelegateNode, DelegateProxyGraph);
					check(GetCapturedVarNode);
					GetCapturedVarNode->VariableReference.SetSelfMember(DelegateInfo->CapturedVariableName);
					GetCapturedVarNode->AllocateDefaultPins();

					// Link our getter node to the original function

					UEdGraphPin* BoundFunctionNodeSelfPin = BoundFunctionNode->FindPinChecked(UEdGraphSchema_K2::PN_Self);
					UEdGraphPin* GetCapturedVarNodeVariablePin = GetCapturedVarNode->FindPinChecked(DelegateInfo->CapturedVariableName, EGPD_Output);

					verify(K2Schema->TryCreateConnection(GetCapturedVarNodeVariablePin, BoundFunctionNodeSelfPin));
				}
			}

			// Finally, update the original node to use our new function

			CreateDelegateNode->SetFunction(DelegateProxyGraph->GetFName());
		}

		ConvertibleDelegates.Remove(SourceCreateDelegateNode);
	}
}

#undef LOCTEXT_NAMESPACE
//////////////////////////////////////////////////////////////////////////
