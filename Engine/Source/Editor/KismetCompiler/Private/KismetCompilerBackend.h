// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"

//////////////////////////////////////////////////////////////////////////
// IKismetCompilerBackend

class IKismetCompilerBackend
{
};

//////////////////////////////////////////////////////////////////////////
// FKismetVMBackend

class FKismetCompilerVMBackend : public IKismetCompilerBackend
{
public:
	typedef TMap<FBlueprintCompiledStatement*, CodeSkipSizeType> TStatementToSkipSizeMap;

protected:
	UBlueprint* Blueprint;
	UEdGraphSchema_K2* Schema;
	FCompilerResultsLog& MessageLog;
	FKismetCompilerContext& CompilerContext;

	TStatementToSkipSizeMap UbergraphStatementLabelMap;
public:
	FKismetCompilerVMBackend(UBlueprint* InBlueprint, UEdGraphSchema_K2* InSchema, FKismetCompilerContext& InContext)
		: Blueprint(InBlueprint)
		, Schema(InSchema)
		, MessageLog(InContext.MessageLog)
		, CompilerContext(InContext)
	{
	}

	void GenerateCodeFromClass(UClass* SourceClass, TIndirectArray<FKismetFunctionContext>& Functions, bool bGenerateStubsOnly=false);

protected:
	/** Builds both the header declaration and body implementation of a function */
	void ConstructFunction(FKismetFunctionContext& FunctionContext, bool bIsUbergraph, bool bGenerateStubOnly);
};

//////////////////////////////////////////////////////////////////////////
