// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	KismetCompilerModule.cpp
=============================================================================*/

#include "KismetCompilerModule.h"
#include "Stats/Stats.h"
#include "UObject/GarbageCollection.h"
#include "UObject/Class.h"
#include "Engine/Blueprint.h"
#include "Stats/StatsMisc.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectIterator.h"
#include "UObject/MetaData.h"
#include "Animation/AnimBlueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompilerMisc.h"
#include "KismetCompiler.h"

#include "Kismet2/KismetDebugUtilities.h"
#include "Kismet2/KismetReinstanceUtilities.h"
#include "Kismet2/BlueprintEditorUtils.h"

#include "UserDefinedStructureCompilerUtils.h"
#include "StructUtils/UserDefinedStruct.h"
#include "IMessageLogListing.h"
#include "Engine/Engine.h"

DEFINE_LOG_CATEGORY(LogK2Compiler);

#define LOCTEXT_NAMESPACE "KismetCompiler"

//////////////////////////////////////////////////////////////////////////
// FKismet2CompilerModule - The Kismet 2 Compiler module
class FKismet2CompilerModule : public IKismetCompilerInterface
{
public:
	// Implementation of the IKismetCompilerInterface
	virtual void RefreshVariables(UBlueprint* Blueprint) override final;
	virtual void CompileStructure(class UUserDefinedStruct* Struct, FCompilerResultsLog& Results) override;
	virtual void RecoverCorruptedBlueprint(class UBlueprint* Blueprint) override;
	virtual void RemoveBlueprintGeneratedClasses(class UBlueprint* Blueprint) override;
	virtual TArray<IBlueprintCompiler*>& GetCompilers() override { return Compilers; }
	virtual void OverrideBPTypeForClass(UClass* Class, TSubclassOf<UBlueprint> BlueprintType) override;
	virtual void OverrideBPTypeForClassInEditor(UClass* Class, TSubclassOf<UBlueprint> BlueprintType) override;
	virtual void OverrideBPGCTypeForBPType(TSubclassOf<UBlueprint> BlueprintType, TSubclassOf<UBlueprintGeneratedClass> BPGCType) override;
	virtual void ValidateBPAndClassType(UBlueprint* BP, FCompilerResultsLog& OutResults) override;
	virtual void GetBlueprintTypesForClass(UClass* ParentClass, UClass*& OutBlueprintClass, UClass*& OutBlueprintGeneratedClass) const override;
	virtual void GetSubclassesWithDifferingBlueprintTypes(UClass* Class, TSet<const UClass*>& OutMismatchedSubclasses) const override;
	// End implementation

	static TSubclassOf<UBlueprint> FindBlueprintType(UClass* ForClass, const TMap<FTopLevelAssetPath, TSubclassOf<UBlueprint>>& FromMap);
private:
	// these are all pointers to native reflection data, so don't require gc visibility
	// this will frustrate hotreload, though - hot reload of objects used as keys or values
	// doesn't really work anyway:
	TMap<FTopLevelAssetPath, TSubclassOf<UBlueprint>> ClassToBPType;
	TMap<FTopLevelAssetPath, TSubclassOf<UBlueprint>> ClassToEditorBPType;
	TMap<TSubclassOf<UBlueprint>, TSubclassOf<UBlueprintGeneratedClass>> BPTypeToBPGCType;

	TArray<IBlueprintCompiler*> Compilers;
};

IMPLEMENT_MODULE( FKismet2CompilerModule, KismetCompiler );

// Compiles a blueprint.

void FKismet2CompilerModule::CompileStructure(UUserDefinedStruct* Struct, FCompilerResultsLog& Results)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FKismet2CompilerModule::CompileStructure);
	Results.SetSourcePath(Struct->GetPathName());
	FUserDefinedStructureCompilerUtils::CompileStruct(Struct, Results, true);
}

void FKismet2CompilerModule::RefreshVariables(UBlueprint* Blueprint)
{
#if 0 // disabled because this is a costly ensure:
	//  ensure that nothing is using this class cause that would be super bad:
	TArray<UObject*> Instances;
	GetObjectsOfClass(*(Blueprint->GeneratedClass), Instances);
	ensure(Instances.Num() == 0 || Instances.Num() == 1);
#endif

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	FCompilerResultsLog MessageLog;

	bool bMissingProperty = false;
	TArray<int32> MissingVariables;

	for (int32 VarIndex = 0; VarIndex < Blueprint->NewVariables.Num(); ++VarIndex)
	{
		FProperty* ExistingVariable = Blueprint->GeneratedClass->FindPropertyByName(Blueprint->NewVariables[VarIndex].VarName);
		if( ExistingVariable == nullptr || ExistingVariable->GetOwner<UObject>() != Blueprint->GeneratedClass)
		{
			MissingVariables.Add(VarIndex);
		}
	}

	if(MissingVariables.Num() != 0)
	{
		UObject* OldCDO = Blueprint->GeneratedClass->GetDefaultObject(false);
		auto Reinstancer = FBlueprintCompileReinstancer::Create(*Blueprint->GeneratedClass);
		ensure(OldCDO->GetClass() != *Blueprint->GeneratedClass);
		// move old cdo aside:
		if(OldCDO)
		{
			OldCDO->Rename(NULL, GetTransientPackage(), REN_DontCreateRedirectors | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
		}
		Blueprint->GeneratedClass->SetDefaultObject(nullptr);

		// add missing properties:
		for( int32 MissingVarIndex : MissingVariables)
		{
			FProperty* NewProperty = FKismetCompilerUtilities::CreatePropertyOnScope(
				Blueprint->GeneratedClass, 
				Blueprint->NewVariables[MissingVarIndex].VarName, 
				Blueprint->NewVariables[MissingVarIndex].VarType, 
				Blueprint->GeneratedClass, 
				CPF_None,
				K2Schema,
				MessageLog);

			if(NewProperty)
			{
				NewProperty->PropertyLinkNext = Blueprint->GeneratedClass->PropertyLink;
				Blueprint->GeneratedClass->PropertyLink = NewProperty;
			}
		}
		
		Blueprint->GeneratedClass->Bind();
		Blueprint->GeneratedClass->StaticLink(true);
		// regenerate CDO:
		Blueprint->GeneratedClass->GetDefaultObject();

		// cpfuo:
		UEngine::FCopyPropertiesForUnrelatedObjectsParams Params;
		Params.bNotifyObjectReplacement = true;
		UEngine::CopyPropertiesForUnrelatedObjects(OldCDO, Blueprint->GeneratedClass->GetDefaultObject(false), Params);
	}
}

// Recover a corrupted blueprint
void FKismet2CompilerModule::RecoverCorruptedBlueprint(class UBlueprint* Blueprint)
{
	UPackage* Package = Blueprint->GetOutermost();

	// Get rid of any stale classes
	for (FThreadSafeObjectIterator ObjIt; ObjIt; ++ObjIt)
	{
		UObject* TestObject = *ObjIt;
		if (TestObject->GetOuter() == Package)
		{
			// This object is in the blueprint package; is it expected?
			if (UClass* TestClass = Cast<UClass>(TestObject))
			{
				if ((TestClass != Blueprint->SkeletonGeneratedClass) && (TestClass != Blueprint->GeneratedClass))
				{
					// Unexpected UClass
					FKismetCompilerUtilities::ConsignToOblivion(TestClass, false);
				}
			}
		}
	}

	CollectGarbage( GARBAGE_COLLECTION_KEEPFLAGS );
}

void FKismet2CompilerModule::RemoveBlueprintGeneratedClasses(class UBlueprint* Blueprint)
{
	if (Blueprint != NULL)
	{
		// Order unfortunately matters, as we want to allow UBlueprintGeneratedClass::GetAuthoritativeClass to function
		// correctly for as long as possible:
		if (Blueprint->SkeletonGeneratedClass != NULL)
		{
			FKismetCompilerUtilities::ConsignToOblivion(Blueprint->SkeletonGeneratedClass, Blueprint->bIsRegeneratingOnLoad);
			Blueprint->SkeletonGeneratedClass = NULL;
		}

		if (Blueprint->GeneratedClass != NULL)
		{
			FKismetCompilerUtilities::ConsignToOblivion(Blueprint->GeneratedClass, Blueprint->bIsRegeneratingOnLoad);
			Blueprint->GeneratedClass = NULL;
		}
	}
}

void FKismet2CompilerModule::OverrideBPTypeForClass(UClass* Class, TSubclassOf<UBlueprint> BlueprintType)
{
	check(Class && BlueprintType);
	#if DO_CHECK
	if (const TSubclassOf<UBlueprint>* ExistingBlueprintType = ClassToBPType.Find(Class->GetClassPathName()))
	{
		ensureMsgf(false,
			TEXT("Ambiguous mapping attempting to add %s to %s when mapping to %s exists"),
			*(Class->GetFullName()), *(BlueprintType.Get()->GetFullName()),
			*(ExistingBlueprintType->Get()->GetFullName())
		);
	}
	#endif // DO_CHECK

	ClassToBPType.Add(Class->GetClassPathName(), BlueprintType);
}

void FKismet2CompilerModule::OverrideBPTypeForClassInEditor(UClass* Class, TSubclassOf<UBlueprint> BlueprintType)
{
	check(Class && BlueprintType);
#if DO_CHECK
	if (const TSubclassOf<UBlueprint>* ExistingBlueprintType = ClassToEditorBPType.Find(Class->GetClassPathName()))
	{
		ensureMsgf(false,
			TEXT("Ambiguous mapping attempting to add %s to %s when mapping to %s exists"),
			*(Class->GetFullName()), *(BlueprintType.Get()->GetFullName()),
			*(ExistingBlueprintType->Get()->GetFullName())
		);
	}
#endif // DO_CHECK

	ClassToEditorBPType.Add(Class->GetClassPathName(), BlueprintType);
}

void FKismet2CompilerModule::OverrideBPGCTypeForBPType(TSubclassOf<UBlueprint> BlueprintType, TSubclassOf<UBlueprintGeneratedClass> BPGCType)
{
	check(BlueprintType && BPGCType);
#if DO_CHECK
	if (const TSubclassOf<UBlueprintGeneratedClass>* ExistingBlueprintType = BPTypeToBPGCType.Find(BlueprintType))
	{
		ensureMsgf(false,
			TEXT("Ambiguous mapping attempting to add %s to %s when mapping to %s exists"),
			*(BlueprintType.Get()->GetFullName()), *(BPGCType.Get()->GetFullName()),
			*(ExistingBlueprintType->Get()->GetFullName())
		);
	}
#endif // DO_CHECK

	BPTypeToBPGCType.Add(BlueprintType, BPGCType);
}

void FKismet2CompilerModule::ValidateBPAndClassType(UBlueprint* BP, FCompilerResultsLog& OutResults)
{
	// validation can become a warning as this matures, for now just note:

	if (BP->BlueprintType == BPTYPE_MacroLibrary)
	{
		// macros will contain macros of all sorts of UClasses, they only have a notional
		// type for scoping purposes, it is not useful to inspect their GeneratedClass:
		return;
	}

	UBlueprintGeneratedClass* BPGC = Cast<UBlueprintGeneratedClass>(BP->GeneratedClass);
	if(!BPGC)
	{
		return;
	}

	// validate according to ClassToBPType:
	{
		TSubclassOf<UBlueprint> ExpectedType = FindBlueprintType(BPGC, ClassToBPType);
		if(!BP->GetClass()->IsChildOf(ExpectedType))
		{
			OutResults.Note(
				*(FText::Format(
					LOCTEXT("BPGCTypeMismatch_Blueprint", "@@ has an incorrect BP type - this type of blueprint ({0}) needs to be converted."),
					FText::FromString(BP->GetFullName())).ToString()),
				BP
			);
		}
	}

	// validate according to ClassToEditorBPType:
	if (::IsEditorOnlyObject(BP->GetOutermost()))
	{
		TSubclassOf<UBlueprint> ExpectedType = FindBlueprintType(BPGC, ClassToEditorBPType);
		if (!BP->GetClass()->IsChildOf(ExpectedType))
		{
			OutResults.Note(
				*(FText::Format(
					LOCTEXT("BPGCTypeMismatch_EditorOnlyBlueprint", "@@ has an incorrect editor BP type - this type of blueprint ({0}) needs to be converted."),
					FText::FromString(BP->GetFullName())).ToString()),
				BP
			);
		}
	}

	// validate according to BPTypeToBPGCType:
	for (TPair<TSubclassOf<UBlueprint>, TSubclassOf<UBlueprintGeneratedClass>> BPTypeToBPGCTypeIter : BPTypeToBPGCType)
	{
		// if there's a BP type implied by the BPGC we want it to match the provided BP:
		if(BP->GetClass()->IsChildOf(BPTypeToBPGCTypeIter.Key))
		{
			if (!BPGC->GetClass()->IsChildOf(BPTypeToBPGCTypeIter.Value))
			{
				OutResults.Note(
					*(FText::Format(
						LOCTEXT("BPGCTypeMismatch_GeneratedClass", "@@ has an incorrect BPGC type - this type of blueprint ({0}) needs to sanitize its class."),
						FText::FromString(BP->GetFullName())).ToString()),
					BP
				);
			}
		}

		// if there's a class implied by the BP type we want it to match the provided BPGC:
		if(BPGC->GetClass()->IsChildOf(BPTypeToBPGCTypeIter.Value))
		{
			if (!BP->GetClass()->IsChildOf(BPTypeToBPGCTypeIter.Key))
			{
				OutResults.Note(
					*(FText::Format(
						LOCTEXT("BPGCTypeMismatch_ClassGeneratedBy", "@@ has an incorrect BP type - this blueprint ({0}) needs to be converted to a different type of blueprint."),
						FText::FromString(BP->GetFullName())).ToString()),
					BP
				);
			}
		}
	}
}

void FKismet2CompilerModule::GetBlueprintTypesForClass(UClass* ParentClass, UClass*& OutBlueprintClass, UClass*& OutBlueprintGeneratedClass) const
{
	const TMap<TSubclassOf<UBlueprint>, TSubclassOf<UBlueprintGeneratedClass>>& BPTypeToBPGCTypeForLookup = BPTypeToBPGCType;
	const auto GetBPGCTypeForBPType = [&BPTypeToBPGCTypeForLookup](TSubclassOf<UBlueprint> BPType) -> TSubclassOf<UBlueprintGeneratedClass>
	{
		if(const TSubclassOf<UBlueprintGeneratedClass>* BPGCType = BPTypeToBPGCTypeForLookup.Find(BPType))
		{
			return *BPGCType;
		}
		return UBlueprintGeneratedClass::StaticClass();
	};

	// honor ClassToEditorBPType
	if(::IsEditorOnlyObject(ParentClass))
	{
		TSubclassOf<UBlueprint> BlueprintType = FindBlueprintType(ParentClass, ClassToEditorBPType);
		if(BlueprintType != UBlueprint::StaticClass())
		{
			OutBlueprintGeneratedClass = GetBPGCTypeForBPType(BlueprintType);
			OutBlueprintClass = BlueprintType;
			return;
		}
	}

	// no editor mapping found, fall back to ClassToBPType:
	TSubclassOf<UBlueprint> BlueprintType = FindBlueprintType(ParentClass, ClassToBPType);
	if (BlueprintType != UBlueprint::StaticClass())
	{
		OutBlueprintGeneratedClass = GetBPGCTypeForBPType(BlueprintType);
		OutBlueprintClass = BlueprintType;
		return;
	}

	// legacy support for IBlueprintCompiler:
	for ( IBlueprintCompiler* Compiler : Compilers )
	{
		if ( Compiler->GetBlueprintTypesForClass(ParentClass, OutBlueprintClass, OutBlueprintGeneratedClass) )
		{
			return;
		}
	}

	OutBlueprintClass = UBlueprint::StaticClass();
	OutBlueprintGeneratedClass = UBlueprintGeneratedClass::StaticClass();
}

void FKismet2CompilerModule::GetSubclassesWithDifferingBlueprintTypes(UClass* Class, TSet<const UClass*>& OutMismatchedSubclasses) const
{
	UClass* BPClass;
	UClass* BPGeneratedClass;
	GetBlueprintTypesForClass(Class, BPClass, BPGeneratedClass);
	
	auto CheckClassToBPTypeMap = [](const TMap<FTopLevelAssetPath, TSubclassOf<UBlueprint>>& Map, const UClass* Class, const UClass* BPClass, TSet<const UClass*>& Result)
	{
		for (const TTuple<FTopLevelAssetPath, TSubclassOf<UBlueprint>>& Pair : Map)
		{
			if (const UClass* SupportedClass = FindObject<UClass>(Pair.Key);
				Pair.Value != BPClass && SupportedClass && SupportedClass->IsChildOf(Class))
			{
				Result.Add(SupportedClass);
			}
		}
	};
	CheckClassToBPTypeMap(ClassToBPType, Class, BPClass, OutMismatchedSubclasses);
	CheckClassToBPTypeMap(ClassToEditorBPType, Class, BPClass, OutMismatchedSubclasses);
}

TSubclassOf<UBlueprint> FKismet2CompilerModule::FindBlueprintType(UClass* ForClass, const TMap<FTopLevelAssetPath, TSubclassOf<UBlueprint>>& FromMap)
{
	UClass* Iter = ForClass;
	while (Iter)
	{
		if (const TSubclassOf<UBlueprint>* BPType = FromMap.Find(Iter->GetClassPathName()))
		{
			return *BPType;
		}
		Iter = Iter->GetSuperClass();
	}
	return UBlueprint::StaticClass();
}

#undef LOCTEXT_NAMESPACE
