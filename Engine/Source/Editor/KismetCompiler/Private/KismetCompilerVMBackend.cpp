// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	KismetCompilerVMBackend.cpp
=============================================================================*/

#include "CoreMinimal.h"
#include "Misc/CoreMisc.h"
#include "UObject/Script.h"
#include "UObject/ObjectMacros.h"
#include "Serialization/ArchiveUObject.h"
#include "UObject/Class.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/Interface.h"
#include "UObject/UnrealType.h"
#include "UObject/TextProperty.h"
#include "UObject/FieldPathProperty.h"
#include "UObject/PropertyPortFlags.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "Engine/LatentActionManager.h"
#include "StructUtils/UserDefinedStruct.h"
#include "BPTerminal.h"
#include "EdGraphSchema_K2.h"
#include "K2Node_MacroInstance.h"
#include "BlueprintCompiledStatement.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "KismetCompiledFunctionContext.h"
#include "Misc/FeedbackContext.h"

#include "KismetCompilerMisc.h"
#include "KismetCompilerBackend.h"

#include "Misc/DefaultValueHelper.h"
#include "ProjectUtilities/BuildTargetSet.h"

#include "Kismet2/StructureEditorUtils.h"
#include "Kismet2/KismetDebugUtilities.h"

#include "Internationalization/TextNamespaceUtil.h"
#include "Internationalization/TextPackageNamespaceUtil.h"
#include "Internationalization/StringTableRegistry.h"
#include "Internationalization/StringTableCore.h"
#include "Internationalization/StringTable.h"

#include "Kismet2/BlueprintEditorUtils.h"

#define LOCTEXT_NAMESPACE "KismetCompilerVMBackend"
//////////////////////////////////////////////////////////////////////////
// FScriptBytecodeWriter

//
// Little class for writing to scripts.
//
class FScriptBytecodeWriter : public FArchiveUObject
{
public:
	TArray<uint8>& ScriptBuffer;
public:
	FScriptBytecodeWriter( TArray<uint8>& InScriptBuffer )
		: ScriptBuffer( InScriptBuffer )
	{
	}

	void Serialize( void* V, int64 Length ) override
	{
		int32 iStart = ScriptBuffer.AddUninitialized(IntCastChecked<int32, int64>(Length));
		FMemory::Memcpy( &(ScriptBuffer[iStart]), V, Length );
	}

	using FArchiveUObject::operator<<; // For visibility of the overloads we don't override

	FArchive& operator<<(FName& Name) override
	{
		FArchive& Ar = *this;

		// This must match the format and endianness expected by XFERNAME
		FNameEntryId ComparisonIndex = Name.GetComparisonIndex(), DisplayIndex = Name.GetDisplayIndex();
		uint32 Number = Name.GetNumber();
		Ar << ComparisonIndex;
		Ar << DisplayIndex;
		Ar << Number;

		return Ar;
	}

	FArchive& operator<<(UObject*& Res) override
	{
		ScriptPointerType D = (ScriptPointerType)Res;
		FArchive& Ar = *this;

		Ar << D;
		return Ar;
	}

	FArchive& operator<<(FObjectPtr& Res) override
	{
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
		ScriptPointerType D = (ScriptPointerType)Res.GetHandle().PointerOrRef;
#elif UE_WITH_REMOTE_OBJECT_HANDLE
		ScriptPointerType D = (ScriptPointerType)Res.GetHandle().Payload;
#else
		ScriptPointerType D = (ScriptPointerType)Res.GetHandle();
#endif
		FArchive& Ar = *this;

		Ar << D;
		return Ar;
	}
	FArchive& operator<<(FField*& Res) override
	{
		ScriptPointerType D = (ScriptPointerType)Res;
		FArchive& Ar = *this;
		Ar << D;
		return Ar;
	}

	FArchive& operator<<(FLazyObjectPtr& LazyObjectPtr) override
	{
		return FArchive::operator<<(LazyObjectPtr);
	}

	FArchive& operator<<(FSoftObjectPtr& Value) override
	{
		return FArchive::operator<<(Value);
	}

	FArchive& operator<<(FSoftObjectPath& Value) override
	{
		return FArchiveUObject::operator<<(Value);
	}

	FArchive& operator<<(TCHAR* S)
	{
		Serialize(S, FCString::Strlen(S) + 1);
		return *this;
	}

	FArchive& operator<<(EExprToken E)
	{
		checkSlow(E < 0xFF);

		uint8 B = static_cast<uint8>(E);
		Serialize(&B, 1);
		return *this;
	}

	FArchive& operator<<(ECastToken E)
	{
		uint8 B = static_cast<uint8>(E);
		Serialize(&B, 1);
		return *this;
	}

	FArchive& operator<<(EBlueprintTextLiteralType E)
	{
		static_assert(sizeof(__underlying_type(EBlueprintTextLiteralType)) == sizeof(uint8), "EBlueprintTextLiteralType is expected to be a uint8");

		uint8 B = (uint8)E;
		Serialize(&B, 1);
		return *this;
	}

	FArchive& operator<<(EPropertyType E)
	{
		uint8 B = static_cast<uint8>(E);
		Serialize(&B, 1);
		return *this;
	}

	CodeSkipSizeType EmitPlaceholderSkip()
	{
		CodeSkipSizeType Result = ScriptBuffer.Num();

		CodeSkipSizeType Placeholder = -1;
		(*this) << Placeholder;

		return Result;
	}

	void CommitSkip(CodeSkipSizeType WriteOffset, CodeSkipSizeType NewValue)
	{
		//@TODO: Any endian issues?
#if SCRIPT_LIMIT_BYTECODE_TO_64KB
		static_assert(sizeof(CodeSkipSizeType) == 2, "Update this code as size changed.");
		ScriptBuffer[WriteOffset] = NewValue & 0xFF;
		ScriptBuffer[WriteOffset+1] = (NewValue >> 8) & 0xFF;
#else
		static_assert(sizeof(CodeSkipSizeType) == 4, "Update this code as size changed.");
		ScriptBuffer[WriteOffset] = NewValue & 0xFF;
		ScriptBuffer[WriteOffset+1] = (NewValue >> 8) & 0xFF;
		ScriptBuffer[WriteOffset+2] = (NewValue >> 16) & 0xFF;
		ScriptBuffer[WriteOffset+3] = (NewValue >> 24) & 0xFF;
#endif
	}
};

//////////////////////////////////////////////////////////////////////////
// FSkipOffsetEmitter

struct FSkipOffsetEmitter
{
	CodeSkipSizeType SkipWriteIndex;
	CodeSkipSizeType StartIndex;
	TArray<uint8>& Script;

	FSkipOffsetEmitter(TArray<uint8>& InScript)
		: SkipWriteIndex(-1)
		, StartIndex(-1)
		, Script(InScript)
	{
	}

	void Emit()
	{
		SkipWriteIndex = (CodeSkipSizeType)Script.Num();
		StartIndex = SkipWriteIndex;

		// Reserve space
		for (int32 i = 0; i < sizeof(CodeSkipSizeType); ++i)
		{
			Script.Add(0);
		}
	}

	void BeginCounting()
	{
		StartIndex = Script.Num();
	}

	void Commit()
	{
		check(SkipWriteIndex != -1);
		CodeSkipSizeType BytesToSkip = Script.Num() - StartIndex;

		//@TODO: Any endian issues?
#if SCRIPT_LIMIT_BYTECODE_TO_64KB
		static_assert(sizeof(CodeSkipSizeType) == 2, "Update this code as size changed.");
		Script[SkipWriteIndex] = BytesToSkip & 0xFF;
		Script[SkipWriteIndex+1] = (BytesToSkip >> 8) & 0xFF;
#else
		static_assert(sizeof(CodeSkipSizeType) == 4, "Update this code as size changed.");
		Script[SkipWriteIndex] = BytesToSkip & 0xFF;
		Script[SkipWriteIndex+1] = (BytesToSkip >> 8) & 0xFF;
		Script[SkipWriteIndex+2] = (BytesToSkip >> 16) & 0xFF;
		Script[SkipWriteIndex+3] = (BytesToSkip >> 24) & 0xFF;
#endif
	}
};

//////////////////////////////////////////////////////////////////////////
// FCodeSkipInfo

class FCodeSkipInfo
{
public:
	enum ECodeSkipType
	{
		Fixup = 0,
		InstrumentedDelegateFixup
	};
	FCodeSkipInfo(ECodeSkipType TypeIn, FBlueprintCompiledStatement* TargetLabelIn = nullptr, FBlueprintCompiledStatement* SourceLabelIn = nullptr)
		: Type(TypeIn)
		, SourceLabel(SourceLabelIn)
		, TargetLabel(TargetLabelIn)
	{
	}

	ECodeSkipType Type;
	FBlueprintCompiledStatement* SourceLabel;
	FBlueprintCompiledStatement* TargetLabel;
	FName DelegateName;
};

//////////////////////////////////////////////////////////////////////////
// FScriptBuilderBase

class FScriptBuilderBase
{
private:
	FScriptBytecodeWriter Writer;
	UBlueprintGeneratedClass* ClassBeingBuilt;
	UEdGraphSchema_K2* Schema;
	friend class FContextEmitter;

	// Pointers to commonly used structures (found in constructor)
	UScriptStruct* VectorStruct;
	UScriptStruct* Vector3fStruct;
	UScriptStruct* RotatorStruct;
	UScriptStruct* TransformStruct;
	UScriptStruct* LatentInfoStruct;
	UScriptStruct* ProfileStruct;

	FKismetCompilerVMBackend::TStatementToSkipSizeMap StatementLabelMap;
	FKismetCompilerVMBackend::TStatementToSkipSizeMap& UbergraphStatementLabelMap;

	// Fixup list for jump targets (location to overwrite; jump target)
	TMap<CodeSkipSizeType, FCodeSkipInfo> JumpTargetFixupMap;

	// Is this compiling the ubergraph?
	bool bIsUbergraph;

	FBlueprintCompiledStatement& ReturnStatement;

	FKismetCompilerContext* CurrentCompilerContext;
	FKismetFunctionContext* CurrentFunctionContext;

	// Pure node count/starting offset (used for instrumentation)
	int32 PureNodeEntryCount;
	int32 PureNodeEntryStart;

protected:
	/**
	 * This class is designed to be used like so to emit a bytecode context expression:
	 *
	 *   {
	 *       FContextEmitter ContextHandler;
	 *       if (Needs Context)
	 *       {
	 *           ContextHandler.StartContext(context);
	 *       }
	 *       Do stuff predicated on context
	 *       // Emitter closes when it falls out of scope
	 *   }
	 */
	struct FContextEmitter
	{
	private:
		FScriptBuilderBase& ScriptBuilder;
		FScriptBytecodeWriter& Writer;
		TArray<FSkipOffsetEmitter> SkipperStack;
		bool bInContext;
	public:
		FContextEmitter(FScriptBuilderBase& InScriptBuilder)
			: ScriptBuilder(InScriptBuilder)
			, Writer(ScriptBuilder.Writer)
			, bInContext(false)
		{
		}

		/** Starts a context if the Term isn't NULL */
		void TryStartContext(FBPTerminal* Term, bool bUnsafeToSkip = false, bool bIsInterfaceContext = false, FProperty* RValueProperty = nullptr)
		{
			if (Term != NULL)
			{
				StartContext(Term, bUnsafeToSkip, bIsInterfaceContext, RValueProperty);
			}
		}

		void StartContext(FBPTerminal* Term, bool bUnsafeToSkip = false, bool bIsInterfaceContext = false, FProperty* RValueProperty = nullptr)
		{
			bInContext = true;

			if(Term->IsClassContextType())
			{
				Writer << EX_ClassContext;
			}
			else
			{
				static const FBoolConfigValueHelper CanSuppressAccessViolation(TEXT("Kismet"), TEXT("bCanSuppressAccessViolation"), GEngineIni);
				if (bUnsafeToSkip || !CanSuppressAccessViolation)
				{
					Writer << EX_Context;
				}
				else
				{
					Writer << EX_Context_FailSilent;
				}

				if (bIsInterfaceContext)
				{
					Writer << EX_InterfaceContext;
				}
			}

			ScriptBuilder.EmitTerm(Term);

			// Skip offset if the expression evaluates to null (counting from later on)
			FSkipOffsetEmitter Skipper(ScriptBuilder.Writer.ScriptBuffer);
			Skipper.Emit();

			// R-Value property, see ReadVariableSize in UObject::ProcessContextOpcode() for usage
			Writer << RValueProperty;

			// Context expression (this is the part that gets skipped if the object turns out NULL)
			Skipper.BeginCounting();

			SkipperStack.Push( Skipper );
		}

		void CloseContext()
		{
			// Point to skip to (end of sequence)
			for (int32 i = 0; i < SkipperStack.Num(); ++i)
			{
				SkipperStack[i].Commit();
			}

			bInContext = false;
		}

		~FContextEmitter()
		{
			if (bInContext)
			{
				CloseContext();
			}
		}
	};
public:
	FScriptBuilderBase(TArray<uint8>& InScript, UBlueprintGeneratedClass* InClass, UEdGraphSchema_K2* InSchema, FKismetCompilerVMBackend::TStatementToSkipSizeMap& InUbergraphStatementLabelMap, bool bInIsUbergraph, FBlueprintCompiledStatement& InReturnStatement)
		: Writer(InScript)
		, ClassBeingBuilt(InClass)
		, Schema(InSchema)
		, UbergraphStatementLabelMap(InUbergraphStatementLabelMap)
		, bIsUbergraph(bInIsUbergraph)
		, ReturnStatement(InReturnStatement)
		, CurrentCompilerContext(nullptr)
		, CurrentFunctionContext(nullptr)
		, PureNodeEntryCount(0)
		, PureNodeEntryStart(0)
	{
		VectorStruct = TBaseStructure<FVector>::Get();
		Vector3fStruct = TVariantStructure<FVector3f>::Get();
		RotatorStruct = TBaseStructure<FRotator>::Get();
		TransformStruct = TBaseStructure<FTransform>::Get();
		LatentInfoStruct = FLatentActionInfo::StaticStruct();
	}

	void CopyStatementMapToUbergraphMap()
	{
		UbergraphStatementLabelMap = StatementLabelMap;
	}

	void EmitStringLiteral(const FString& String)
	{
		if (FCString::IsPureAnsi(*String))
		{
			Writer << EX_StringConst;
			uint8 OutCh;
			for (const TCHAR* Ch = *String; *Ch; ++Ch)
			{
				OutCh = CharCast<ANSICHAR>(*Ch);
				Writer << OutCh;
			}

			OutCh = 0;
			Writer << OutCh;
		}
		else
		{
			// Note: This is a no-op on platforms that are using a 16-bit TCHAR
			FTCHARToUTF16 UTF16String(*String, String.Len() + 1); // include the null terminator

			Writer << EX_UnicodeStringConst;
			uint16 OutCh;
			for (const UTF16CHAR* Ch = UTF16String.Get(); *Ch; ++Ch)
			{
				OutCh = *Ch;
				Writer << OutCh;
			}

			OutCh = 0;
			Writer << OutCh;
		}
	}

	struct FLiteralTypeHelper
	{
		static bool IsBoolean(const FEdGraphPinType* Type, const FProperty* Property)
		{
			if (Property)
			{
				return Property->IsA<FBoolProperty>();
			}
			return Type && (Type->PinCategory == UEdGraphSchema_K2::PC_Boolean);
		}

		static bool IsBit(const FProperty* Property)
		{
			if (Property && Property->GetOwnerStruct() && !CastFieldChecked<FBoolProperty>(Property)->IsNativeBool())
			{
				return true;
			}
			return false;
		}

		static bool IsString(const FEdGraphPinType* Type, const FProperty* Property)
		{
			if (Property)
			{
				return Property->IsA<FStrProperty>();
			}
			return Type && (Type->PinCategory == UEdGraphSchema_K2::PC_String);
		}

		static bool IsText(const FEdGraphPinType* Type, const FProperty* Property)
		{
			if (Property)
			{
				return Property->IsA<FTextProperty>();
			}
			return Type && (Type->PinCategory == UEdGraphSchema_K2::PC_Text);
		}

		static bool IsFloat(const FEdGraphPinType* Type, const FProperty* Property)
		{
			if (Property)
			{
				return Property->IsA<FFloatProperty>();
			}
			return Type && (Type->PinCategory == UEdGraphSchema_K2::PC_Real) && (Type->PinSubCategory == UEdGraphSchema_K2::PC_Float);
		}

		static bool IsDouble(const FEdGraphPinType* Type, const FProperty* Property)
		{
			if (Property)
			{
				return Property->IsA<FDoubleProperty>();
			}
			return Type && (Type->PinCategory == UEdGraphSchema_K2::PC_Real) && (Type->PinSubCategory == UEdGraphSchema_K2::PC_Double);
		}

		static bool IsInt(const FEdGraphPinType* Type, const FProperty* Property)
		{
			if (Property)
			{
				return Property->IsA<FIntProperty>();
			}
			return Type && (Type->PinCategory == UEdGraphSchema_K2::PC_Int);
		}

		static bool IsInt64(const FEdGraphPinType* Type, const FProperty* Property)
		{
			if (Property)
			{
				return Property->IsA<FInt64Property>();
			}
			return Type && (Type->PinCategory == UEdGraphSchema_K2::PC_Int64);
		}

		static bool IsUInt64(const FEdGraphPinType* Type, const FProperty* Property)
		{
			if (Property)
			{
				return Property->IsA<FUInt64Property>();
			}
			return false;
		}

		static bool IsByte(const FEdGraphPinType* Type, const FProperty* Property)
		{
			if (Property)
			{
				return Property->IsA<FByteProperty>() || (Property->IsA<FEnumProperty>() && static_cast<const FEnumProperty*>(Property)->GetUnderlyingProperty()->IsA<FByteProperty>());
			}
			return Type && ((Type->PinCategory == UEdGraphSchema_K2::PC_Byte) || (Type->PinCategory == UEdGraphSchema_K2::PC_Enum));
		}

		static bool IsName(const FEdGraphPinType* Type, const FProperty* Property)
		{
			if (Property)
			{
				return Property->IsA<FNameProperty>();
			}
			return Type && (Type->PinCategory == UEdGraphSchema_K2::PC_Name);
		}

		static bool IsStruct(const FEdGraphPinType* Type, const FProperty* Property)
		{
			if (Property)
			{
				return Property->IsA<FStructProperty>();
			}
			return Type && (Type->PinCategory == UEdGraphSchema_K2::PC_Struct);
		}

		static bool IsDelegate(const FEdGraphPinType* Type, const FProperty* Property)
		{
			if (Property)
			{
				return Property->IsA<FDelegateProperty>();
			}
			return Type && (Type->PinCategory == UEdGraphSchema_K2::PC_Delegate);
		}

		static bool IsSoftObject(const FEdGraphPinType* Type, const FProperty* Property)
		{
			if (Property)
			{
				return Property->IsA<FSoftObjectProperty>();
			}
			return Type && (Type->PinCategory == UEdGraphSchema_K2::PC_SoftObject);
		}

		// Will handle Class properties as well
		static bool IsObject(const FEdGraphPinType* Type, const FProperty* Property)
		{
			if (Property)
			{
				return Property->IsA<FObjectPropertyBase>();
			}
			return Type && (Type->PinCategory == UEdGraphSchema_K2::PC_Object);
		}

		static bool IsClass(const FEdGraphPinType* Type, const FProperty* Property)
		{
			if (Property)
			{
				return Property->IsA<FClassProperty>();
			}
			return Type && (Type->PinCategory == UEdGraphSchema_K2::PC_Class);
		}

		static bool IsInterface(const FEdGraphPinType* Type, const FProperty* Property)
		{
			if (Property)
			{
				return Property->IsA<FInterfaceProperty>();
			}
			return Type && (Type->PinCategory == UEdGraphSchema_K2::PC_Interface);
		}

		static bool IsFieldPath(const FEdGraphPinType* Type, const FProperty* Property)
		{
			if (Property)
			{
				return Property->IsA<FFieldPathProperty>();
			}
			return Type && (Type->PinCategory == UEdGraphSchema_K2::PC_FieldPath);
		}
	};

	void EmitTermExpr(FBPTerminal* Term, const FProperty* CoerceProperty = NULL, bool bAllowStaticArray = false, bool bCallerRequiresBit = false)
	{
		constexpr EPropertyPortFlags ImportTextPortFlags = EPropertyPortFlags::PPF_SerializedAsImportText;
		if (Term->bIsLiteral)
		{
			check(!Term->Type.IsContainer() || CoerceProperty);

			// Additional Validation, since we cannot trust custom k2nodes
			if (CoerceProperty && ensure(Schema) && ensure(CurrentCompilerContext))
			{
			    const bool bSpecialCaseSelf = (Term->Type.PinSubCategory == UEdGraphSchema_K2::PN_Self);
				if(!bSpecialCaseSelf)
			    {
				    FEdGraphPinType TrueType;
				    const bool bValidProperty = Schema->ConvertPropertyToPinType(CoerceProperty, TrueType);

				    auto AreTypesBinaryCompatible = [](const FEdGraphPinType& TypeA, const FEdGraphPinType& TypeB) -> bool
				    {
					    if (TypeA.PinCategory != TypeB.PinCategory)
					    {
						    return false;
					    }
					    if ((TypeA.ContainerType != TypeB.ContainerType)
						    || (TypeA.bIsWeakPointer != TypeB.bIsWeakPointer))
					    {
						    return false;
					    }
					    if (TypeA.PinCategory == UEdGraphSchema_K2::PC_Struct)
					    {
						    if (TypeA.PinSubCategoryObject != TypeB.PinSubCategoryObject)
						    {
							    return false;
						    }
					    }
					    return true;
				    };

					if (bValidProperty && !AreTypesBinaryCompatible(Term->Type, TrueType))
					{
						const FString ErrorMessage = FString::Printf(TEXT("ICE: The type of property %s doesn't match the terminal type for pin @@."), *CoerceProperty->GetPathName());
						CurrentCompilerContext->MessageLog.Error(*ErrorMessage, Term->SourcePin);
					}
				}
			}

			if (FLiteralTypeHelper::IsString(&Term->Type, CoerceProperty))
			{
				EmitStringLiteral(Term->Name);
			}
			else if (FLiteralTypeHelper::IsText(&Term->Type, CoerceProperty))
			{
				Writer << EX_TextConst;

				const FString& StringValue = FTextInspector::GetDisplayString(Term->TextLiteral);

				// What kind of text are we dealing with?
				if (Term->TextLiteral.IsEmpty())
				{
					Writer << EBlueprintTextLiteralType::Empty;
				}
				else if (Term->TextLiteral.IsFromStringTable())
				{
					FName TableId;
					FString Key;
					FTextInspector::GetTableIdAndKey(Term->TextLiteral, TableId, Key);

					UStringTable* StringTableAsset = FStringTableRegistry::Get().FindStringTableAsset(TableId);

					Writer << EBlueprintTextLiteralType::StringTableEntry;
					Writer << StringTableAsset; // Not used at runtime, but exists for asset dependency gathering
					EmitStringLiteral(TableId.ToString());
					EmitStringLiteral(Key);
				}
				else if (Term->TextLiteral.IsCultureInvariant())
				{
					Writer << EBlueprintTextLiteralType::InvariantText;
					EmitStringLiteral(StringValue);
				}
				else
				{
					FTextId TextId;
					const FString* SourceString = FTextInspector::GetSourceString(Term->TextLiteral);

					if (SourceString && Term->TextLiteral.ShouldGatherForLocalization())
					{
						TextId = FTextInspector::GetTextId(Term->TextLiteral);
					}

					if (!TextId.IsEmpty())
					{
						// BP bytecode always removes the package localization ID to match how text works at runtime
						// If we're gathering editor-only text then we'll pick up the version with the package localization ID from the property/pin rather than the bytecode
						const FString Namespace = TextNamespaceUtil::StripPackageNamespace(TextId.GetNamespace().ToString());
						const FString Key = TextId.GetKey().ToString();
#if WITH_EDITORONLY_DATA
						const FString& DevNotes = Term->TextLiteral.GetDevNotes();
						const bool bEmitDevNotes = !Writer.IsFilterEditorOnly() && !DevNotes.IsEmpty();
#else
						constexpr bool bEmitDevNotes = false;
#endif

						Writer << (bEmitDevNotes ? EBlueprintTextLiteralType::LocalizedTextWithNotes : EBlueprintTextLiteralType::LocalizedText);
						EmitStringLiteral(*SourceString);
						EmitStringLiteral(Key);
						EmitStringLiteral(Namespace);
#if WITH_EDITORONLY_DATA
						if (bEmitDevNotes)
						{
							EmitStringLiteral(DevNotes);
						}
#endif
					}
					else
					{
						Writer << EBlueprintTextLiteralType::LiteralString;
						EmitStringLiteral(StringValue);
					}
				}
			}
			else if (FLiteralTypeHelper::IsFloat(&Term->Type, CoerceProperty))
			{
				float Value = FCString::Atof(*(Term->Name));
				Writer << EX_FloatConst;
				Writer << Value;
			}
			else if (FLiteralTypeHelper::IsDouble(&Term->Type, CoerceProperty))
			{
				double Value = 0.0;
				if (Term->Type.bSerializeAsSinglePrecisionFloat)
				{
					Value = FCString::Atof(*(Term->Name));
				}
				else
				{
					Value = FCString::Atod(*(Term->Name));
				}

				Writer << EX_DoubleConst;
				Writer << Value;
			}
			else if (FLiteralTypeHelper::IsInt(&Term->Type, CoerceProperty))
			{
				// In certain cases (like UKismetArrayLibrary functions), we have
				// polymorphic functions that provide their own "custom thunk"
				// (custom execution code). The actual function acts as a
				// template, where the parameter types can be changed out for
				// other types (much like c++ template functions, the "custom
				// thunk" is generic). Traditionally, we use integer refs as the
				// place holder type (that's why this is nested in a FIntProperty
				// check)... Complications arise here, when we try to emit
				// literal values fed into the function when they don't match
				// the template's (int) type. For most types, this here is
				// circumvented with AutoCreateRefTerm, but when it is a self
				// (literal) node we still end up here. So, we try to detect and
				// handle that case here.
				if ((Term->Type.PinSubCategory == UEdGraphSchema_K2::PN_Self) && CoerceProperty && CoerceProperty->HasAnyPropertyFlags(CPF_ReferenceParm))
				{
					Writer << EX_Self;
				}
				else
				{
					//@TODO: There are smaller encodings EX_IntZero, EX_IntOne, EX_IntConstByte available which could be used instead when the value fits
					int32 Value = FCString::Atoi(*(Term->Name));
					Writer << EX_IntConst;
					Writer << Value;
				}
			}
			else if (FLiteralTypeHelper::IsInt64(&Term->Type, CoerceProperty))
			{
				int64 Value = 0;
				LexFromString(Value, *(Term->Name));
				Writer << EX_Int64Const;
				Writer << Value;
			}
			else if (FLiteralTypeHelper::IsUInt64(&Term->Type, CoerceProperty))
			{
				uint64 Value = 0;
				LexFromString(Value, *(Term->Name));
				Writer << EX_UInt64Const;
				Writer << Value;
			}
			else if (FLiteralTypeHelper::IsByte(&Term->Type, CoerceProperty))
			{
				uint8 Value = 0;

				UEnum* EnumPtr = nullptr;

				if (const FByteProperty* ByteProp = CastField< FByteProperty >(CoerceProperty))
				{
					EnumPtr = ByteProp->Enum;
				}
				else if (const FEnumProperty* EnumProp = CastField< FEnumProperty >(CoerceProperty))
				{
					EnumPtr = EnumProp->GetEnum();
				}

				//Parameter property can represent a generic byte. we need the actual type to parse the value.
				if (!EnumPtr)
				{
					EnumPtr = Cast<UEnum>(Term->Type.PinSubCategoryObject.Get());
				}

				//Check for valid enum object reference
				if (EnumPtr)
				{
					//Get value from enum string
					Value = EnumPtr->GetValueByName(*(Term->Name));
				}
				else
				{
					Value = FCString::Atoi(*(Term->Name));
				}

				Writer << EX_ByteConst;
				Writer << Value;
			}
			else if (FLiteralTypeHelper::IsBoolean(&Term->Type, CoerceProperty))
			{
				// Bitfields in struct literals were being treated as full bytes, but instructions like
				// EX_LetBool provide the destination as the CoerceProperty, even though they are going
				// to allocate a full byte for us to write to. To disambiguate I have added bCallerRequiresBit
				// when a calling expressing allocates only a single bit for us to write to:
				bool bValue = Term->Name.ToBool();
				const bool bIsBit = bCallerRequiresBit && FLiteralTypeHelper::IsBit(CoerceProperty);

				if(bIsBit)
				{
					check(CoerceProperty);
					// FArchive const correctness workaround:
					FProperty* BitProperty = const_cast<FProperty*>(CoerceProperty);
					uint8 ValueAsByte = bValue;

					// Emit the literal, with enough information to safely write to CoerceProperty:
					Writer << EX_BitFieldConst;
					Writer << BitProperty;
					Writer << ValueAsByte;
				}
				else
				{
					Writer << (bValue ? EX_True : EX_False);
				}
			}
			else if (FLiteralTypeHelper::IsName(&Term->Type, CoerceProperty))
			{
				FName LiteralName(*(Term->Name));
				Writer << EX_NameConst;
				Writer << LiteralName;
			}
			else if (FLiteralTypeHelper::IsStruct(&Term->Type, CoerceProperty))
			{
				const FStructProperty* StructProperty = CastField<FStructProperty>(CoerceProperty);
				UScriptStruct* Struct = StructProperty ? ToRawPtr(StructProperty->Struct) : ToRawPtr(Cast<UScriptStruct>(Term->Type.PinSubCategoryObject.Get()));
				check(Struct);

				if (Struct == VectorStruct)
				{
					FVector V = FVector::ZeroVector;
					if (!Term->Name.IsEmpty())
					{
						const bool bParsedUsingCustomFormat = FDefaultValueHelper::ParseVector(Term->Name, /*out*/ V);
						if (!bParsedUsingCustomFormat)
						{
							Struct->ImportText(*Term->Name, &V, nullptr, ImportTextPortFlags, GWarn, GetPathNameSafe(StructProperty));
						}
					}
					Writer << EX_VectorConst;
					Writer << V;
				}
				else if (Struct == Vector3fStruct)
				{
					FVector3f V = FVector3f::ZeroVector;
					if (!Term->Name.IsEmpty())
					{
						const bool bParsedUsingCustomFormat = FDefaultValueHelper::ParseVector(Term->Name, /*out*/ V);
						if (!bParsedUsingCustomFormat)
						{
							Struct->ImportText(*Term->Name, &V, nullptr, ImportTextPortFlags, GWarn, GetPathNameSafe(StructProperty));
						}
					}
					Writer << EX_Vector3fConst;
					Writer << V;
				}
				else if (Struct == RotatorStruct)
				{
					FRotator R = FRotator::ZeroRotator;
					if (!Term->Name.IsEmpty())
					{
						const bool bParsedUsingCustomFormat = FDefaultValueHelper::ParseRotator(Term->Name, /*out*/ R);
						if (!bParsedUsingCustomFormat)
						{
							Struct->ImportText(*Term->Name, &R, nullptr, ImportTextPortFlags, GWarn, GetPathNameSafe(StructProperty));
						}
					}
					Writer << EX_RotationConst;
					Writer << R;
				}
				else if (Struct == TransformStruct)
				{
					FTransform T = FTransform::Identity;
					if (!Term->Name.IsEmpty())
					{
						const bool bParsedUsingCustomFormat = T.InitFromString(Term->Name);
						if (!bParsedUsingCustomFormat)
						{
							Struct->ImportText(*Term->Name, &T, nullptr, ImportTextPortFlags, GWarn, GetPathNameSafe(StructProperty));
						}
					}
					Writer << EX_TransformConst;
					Writer << T;
				}
				else
				{
					const int32 ArrayDim = StructProperty ? StructProperty->ArrayDim : 1; //@TODO: is it safe when StructProperty == nullptr?
					int32 StructSize = Struct->GetStructureSize() * ArrayDim;
					uint8* StructData = (uint8*)FMemory_Alloca(StructSize);
					Struct->InitializeStruct(StructData, ArrayDim);
					if (!ensure(bAllowStaticArray || 1 == ArrayDim))
					{
						UE_LOGF(LogK2Compiler, Error, "Unsupported static array. Property: %ls, Struct: %ls", *GetPathNameSafe(StructProperty), *Struct->GetName());
					}

					// Assume that any errors on the import of the name string have been caught in the function call generation
					Struct->ImportText(Term->Name.IsEmpty() ? TEXT("()") : *Term->Name, StructData, nullptr, ImportTextPortFlags, GLog, GetPathNameSafe(StructProperty));

 					Writer << EX_StructConst;
					Writer << Struct;
					Writer << StructSize;

					// TODO: Change this once structs/classes can be declared as explicitly editor only
					bool bIsEditorOnlyStruct = false;

					checkSlow(Schema);
					for( FProperty* Prop = Struct->PropertyLink; Prop; Prop = Prop->PropertyLinkNext )
					{
						// Skip transient and editor only properties, this needs to be synched with ScriptCore
						if (Prop->PropertyFlags & CPF_Transient || (!bIsEditorOnlyStruct && Prop->PropertyFlags & CPF_EditorOnly))
						{
							continue;
						}

						// Create a new term for each property, and serialize it out
						for (int32 ArrayIter = 0; ArrayIter < Prop->ArrayDim; ++ArrayIter)
						{
							FBPTerminal NewTerm;
							if(!Schema->ConvertPropertyToPinType(Prop, NewTerm.Type))
							{
								// Do nothing for unsupported/unhandled property types. This will leave the value unchanged from its constructed default.
								Writer << EX_Nothing;
								continue;
							}

							NewTerm.bIsLiteral = true;
							NewTerm.Source = Term->Source;
							NewTerm.SourcePin = Term->SourcePin;
							Prop->ExportText_InContainer(ArrayIter, NewTerm.Name, StructData, StructData, NULL, PPF_None);
							if (Prop->IsA(FTextProperty::StaticClass()))
							{
								NewTerm.TextLiteral = CastField<FTextProperty>(Prop)->GetPropertyValue_InContainer(StructData, ArrayIter);
								NewTerm.Name = NewTerm.TextLiteral.ToString();
							}
							else if (Prop->IsA(FObjectProperty::StaticClass()))
							{
								NewTerm.ObjectLiteral = CastField<FObjectProperty>(Prop)->GetObjectPropertyValue(Prop->ContainerPtrToValuePtr<void>(StructData));
							}

							EmitTermExpr(&NewTerm, Prop, true, true);
						}
					}
					Struct->DestroyStruct(StructData, ArrayDim);
					Writer << EX_EndStructConst;
				}
			}
			else if (const FArrayProperty* ArrayPropr = CastField<FArrayProperty>(CoerceProperty))
			{
				FProperty* InnerProp = ArrayPropr->Inner;
				ensure(InnerProp);
				FScriptArray ScriptArray;
				ArrayPropr->ImportText_Direct(*Term->Name, &ScriptArray, NULL, ImportTextPortFlags, GLog);

				FScriptArrayHelper ScriptArrayHelper(ArrayPropr, &ScriptArray);
				int32 ElementNum = ScriptArrayHelper.Num();

				Writer << EX_ArrayConst;
				Writer << InnerProp;
				Writer << ElementNum;
				for (int32 ElemIdx = 0; ElemIdx < ElementNum; ++ElemIdx)
				{
					uint8* RawElemData = ScriptArrayHelper.GetRawPtr(ElemIdx);
					EmitInnerElementExpr(Term, InnerProp, RawElemData);
				}
				Writer << EX_EndArrayConst;
			}
			else if (const FSetProperty* SetPropr = CastField<FSetProperty>(CoerceProperty))
			{
				FProperty* InnerProp = SetPropr->ElementProp;
				ensure(InnerProp);

				FScriptSet ScriptSet;
				SetPropr->ImportText_Direct(*Term->Name, &ScriptSet, NULL, ImportTextPortFlags, GLog);
				int32 ElementNum = ScriptSet.Num();

				FScriptSetHelper ScriptSetHelper(SetPropr, &ScriptSet);

				Writer << EX_SetConst;
				Writer << InnerProp;
				Writer << ElementNum;

				for (FScriptSetHelper::FIterator It(ScriptSetHelper); It; ++It)
				{
					uint8* RawElemData = ScriptSetHelper.GetElementPtr(It);
					EmitInnerElementExpr(Term, InnerProp, RawElemData);
				}
				Writer << EX_EndSetConst;
			}
			else if (const FMapProperty* MapPropr = CastField<FMapProperty>(CoerceProperty))
			{
				FProperty* KeyProp = MapPropr->KeyProp;
				FProperty* ValProp = MapPropr->ValueProp;
				ensure(KeyProp && ValProp);

				FScriptMap ScriptMap;
				MapPropr->ImportText_Direct(*Term->Name, &ScriptMap, NULL, ImportTextPortFlags, GLog);
				int32 ElementNum = ScriptMap.Num();

				FScriptMapHelper ScriptMapHelper(MapPropr, &ScriptMap);

				Writer << EX_MapConst;
				Writer << KeyProp;
				Writer << ValProp;
				Writer << ElementNum;

				for (FScriptMapHelper::FIterator It(ScriptMapHelper); It; ++It)
				{
					EmitInnerElementExpr(Term, KeyProp, ScriptMapHelper.GetKeyPtr(It));
					EmitInnerElementExpr(Term, ValProp, ScriptMapHelper.GetValuePtr(It));
				}
				Writer << EX_EndMapConst;
			}
			else if (FLiteralTypeHelper::IsDelegate(&Term->Type, CoerceProperty))
			{
				FName FunctionName;

				// Deliberately null delegates are allowed, using empty string or the ExportText format
				if (Term->Name != TEXT("") && Term->Name != TEXT("(null).None"))
				{
					FunctionName = *Term->Name; //@TODO: K2 Delegate Support: Need to verify this function actually exists and has the right signature?
				}

				Writer << EX_InstanceDelegate;
				Writer << FunctionName;
			}
			else if (FLiteralTypeHelper::IsSoftObject(&Term->Type, CoerceProperty))
			{
				Writer << EX_SoftObjectConst;
				EmitStringLiteral(Term->Name);
			}
			else if (FLiteralTypeHelper::IsFieldPath(&Term->Type, CoerceProperty))
			{
				Writer << EX_FieldPathConst;
				EmitStringLiteral(Term->Name);
			}
			else if (FLiteralTypeHelper::IsObject(&Term->Type, CoerceProperty) || FLiteralTypeHelper::IsClass(&Term->Type, CoerceProperty))
			{
				// Note: This case handles both FObjectProperty and FClassProperty
				if (Term->Type.PinSubCategory == UEdGraphSchema_K2::PN_Self)
				{
					Writer << EX_Self;
				}
				else if (!Term->ObjectLiteral)
				{
					Writer << EX_NoObject;
				}
				else
				{
					Writer << EX_ObjectConst;
					Writer << Term->ObjectLiteral;
				}
			}
			else if (FLiteralTypeHelper::IsInterface(&Term->Type, CoerceProperty))
			{
				if (Term->Type.PinSubCategory == UEdGraphSchema_K2::PN_Self)
				{
					Writer << EX_ObjToInterfaceCast;
					UClass* InterfaceClass = CastChecked<UClass>(Term->Type.PinSubCategoryObject.Get());
					Writer << InterfaceClass;
					Writer << EX_Self;
				}
				else if (Term->ObjectLiteral == nullptr)
				{
					Writer << EX_NoInterface;
				}
				else
				{
					// emit cast
					Writer << EX_ObjToInterfaceCast;
					UClass* InterfaceClass = CastChecked<UClass>(Term->Type.PinSubCategoryObject.Get());
					Writer << InterfaceClass;
					// from the object:
					Writer << EX_ObjectConst;
					Writer << Term->ObjectLiteral;
				}
			}
			else if (!CoerceProperty && Term->Type.PinCategory.IsNone() && (Term->Type.PinSubCategory == UEdGraphSchema_K2::PN_Self))
			{
				Writer << EX_Self;
			}
			// else if (CoerceProperty->IsA(FMulticastDelegateProperty::StaticClass()))
			// Cannot assign a literal to a multicast delegate; it should be added instead of assigned
			else
			{
				if (ensure(CurrentCompilerContext))
				{
					FFormatNamedArguments Args;
					Args.Add(TEXT("PropertyType"), CoerceProperty ? CoerceProperty->GetClass()->GetDisplayNameText() : FText());
					Args.Add(TEXT("PropertyName"), CoerceProperty ? CoerceProperty->GetDisplayNameText() : FText());
					CurrentCompilerContext->MessageLog.Error(*FText::Format(LOCTEXT("InvalidProperty", "It is not possible to express this type ({PropertyType}) as a literal value for the property {PropertyName} on pin @@! If it is inside a struct, you can add a Make struct node to resolve this issue!"), Args).ToString(), Term->SourcePin);
				}
			}
		}
		else
		{
			check(Term->AssociatedVarProperty);
			if (Term->IsDefaultVarTerm())
			{
				Writer << EX_DefaultVariable;
			}
			else if (Term->IsLocalVarTerm())
			{
				Writer << (Term->AssociatedVarProperty->HasAnyPropertyFlags(CPF_OutParm) ? EX_LocalOutVariable : EX_LocalVariable);
			}
			else if (Term->IsSparseClassDataVarTerm())
			{
				Writer << EX_ClassSparseDataVariable;
			}
			else
			{
				Writer << EX_InstanceVariable;
			}
			Writer << Term->AssociatedVarProperty;
		}
	}

	void EmitInnerElementExpr(FBPTerminal* OuterTerm, FProperty* InnerProp, uint8* RawElemPtr)
	{
		FBPTerminal NewTerm;
		Schema->ConvertPropertyToPinType(InnerProp, NewTerm.Type);
		NewTerm.bIsLiteral = true;
		NewTerm.Source = OuterTerm->Source;
		NewTerm.SourcePin = OuterTerm->SourcePin;

		InnerProp->ExportText_Direct(NewTerm.Name, RawElemPtr, RawElemPtr, NULL, PPF_None);
		if (InnerProp->IsA(FTextProperty::StaticClass()))
		{
			NewTerm.TextLiteral = CastField<FTextProperty>(InnerProp)->GetPropertyValue(RawElemPtr);
			NewTerm.Name = NewTerm.TextLiteral.ToString();
		}
		else if (InnerProp->IsA(FObjectPropertyBase::StaticClass()))
		{
			NewTerm.ObjectLiteral = CastField<FObjectPropertyBase>(InnerProp)->GetObjectPropertyValue(RawElemPtr);
		}

		EmitTermExpr(&NewTerm, InnerProp);
	}

	void EmitLatentInfoTerm(FBPTerminal* Term, FProperty* LatentInfoProperty, FBlueprintCompiledStatement* TargetLabel)
	{
		// Special case of the struct property emitter.  Needs to emit a linkage property for fixup
		FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(LatentInfoProperty);
		check(StructProperty->Struct == LatentInfoStruct);

		int32 StructSize = LatentInfoStruct->GetStructureSize();
		uint8* StructData = (uint8*)FMemory_Alloca(StructSize);
		StructProperty->InitializeValue(StructData);

		// Assume that any errors on the import of the name string have been caught in the function call generation
		StructProperty->ImportText_Direct(*Term->Name, StructData, NULL, 0, GLog);

		Writer << EX_StructConst;
		Writer << LatentInfoStruct;
		Writer << StructSize;

		checkSlow(Schema);
		for (FProperty* Prop = LatentInfoStruct->PropertyLink; Prop; Prop = Prop->PropertyLinkNext)
		{
			if (TargetLabel && Prop->GetBoolMetaData(FBlueprintMetadata::MD_NeedsLatentFixup))
			{
				// Emit the literal and queue a fixup to correct it once the address is known
				Writer << EX_SkipOffsetConst;
				CodeSkipSizeType PatchUpNeededAtOffset = Writer.EmitPlaceholderSkip();
				JumpTargetFixupMap.Add(PatchUpNeededAtOffset, FCodeSkipInfo(FCodeSkipInfo::Fixup, TargetLabel));
			}
			else if (Prop->GetBoolMetaData(FBlueprintMetadata::MD_LatentCallbackTarget))
			{
				FBPTerminal CallbackTargetTerm;
				CallbackTargetTerm.bIsLiteral = true;
				CallbackTargetTerm.Type.PinSubCategory = UEdGraphSchema_K2::PN_Self;
				EmitTermExpr(&CallbackTargetTerm, Prop);
			}
			else
			{
				// Create a new term for each property, and serialize it out
				FBPTerminal NewTerm;
				if(Schema->ConvertPropertyToPinType(Prop, NewTerm.Type))
				{
					NewTerm.bIsLiteral = true;
					Prop->ExportText_InContainer(0, NewTerm.Name, StructData, StructData, NULL, PPF_None);

					EmitTermExpr(&NewTerm, Prop);
				}
				else
				{
					// Do nothing for unsupported/unhandled property types. This will leave the value unchanged from its constructed default.
					Writer << EX_Nothing;
				}
			}
		}

		Writer << EX_EndStructConst;
	}

	void EmitFunctionCall(FKismetCompilerContext& CompilerContext, FKismetFunctionContext& FunctionContext, FBlueprintCompiledStatement& Statement, UEdGraphNode* SourceNode)
	{
		UFunction* FunctionToCall = Statement.FunctionToCall;
		check(FunctionToCall);

		ClassBeingBuilt->CalledFunctions.Emplace(FunctionToCall);

		if (FunctionToCall->HasAllFunctionFlags(FUNC_Native))
		{
			// Array output parameters are cleared, in case the native function doesn't clear them before filling.
			int32 NumParams = 0;
			for (TFieldIterator<FProperty> PropIt(FunctionToCall); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
			{
				FProperty* Param = *PropIt;
				check(Param);
				const bool bShouldParameterBeCleared = Param->IsA<FArrayProperty>()
					&& Param->HasAllPropertyFlags(CPF_Parm | CPF_OutParm)
					&& !Param->HasAnyPropertyFlags(CPF_ReferenceParm | CPF_ConstParm | CPF_ReturnParm);
				if (bShouldParameterBeCleared)
				{
					// SetArray instruction will be called with empty parameter list.
					Writer << EX_SetArray;
					FBPTerminal* ArrayTerm = Statement.RHS[NumParams];
					ensure(ArrayTerm && !ArrayTerm->bIsLiteral);
					EmitTerm(ArrayTerm, Param);
					Writer << EX_EndArray;
				}
				NumParams += Param->HasAnyPropertyFlags(CPF_ReturnParm) ? 0 : 1;
			}
		}

		// The target label will only ever be set on a call function when calling into the Ubergraph, which requires a patchup
		// or when re-entering from a latent function which requires a different kind of patchup
		if ((Statement.TargetLabel != NULL) && !bIsUbergraph)
		{
			CodeSkipSizeType OffsetWithinUbergraph = UbergraphStatementLabelMap.FindChecked(Statement.TargetLabel);

			// Overwrite RHS(0) text with the state index to kick off
			check(Statement.RHS[Statement.UbergraphCallIndex]->bIsLiteral);
			Statement.RHS[Statement.UbergraphCallIndex]->Name = FString::FromInt(OffsetWithinUbergraph);

#if UE_BLUEPRINT_EVENTGRAPH_FASTCALLS
			// Store optimization data if this is a simple call into the ubergraph
			if (FunctionContext.bIsSimpleStubGraphWithNoParams && CompilerContext.NewClass->UberGraphFunction)
			{
				check(FunctionToCall == CompilerContext.NewClass->UberGraphFunction);
				check(FunctionToCall->ParmsSize == sizeof(int32));

				if ((FunctionToCall->FirstPropertyToInit == nullptr) && (FunctionToCall->PostConstructLink == nullptr))
				{
					FunctionContext.Function->EventGraphFunction = FunctionToCall;
					FunctionContext.Function->EventGraphCallOffset = OffsetWithinUbergraph;
				}
			}
#endif
		}

		// Handle the return value assignment if present
		bool bHasOutputValue = false;
		for (TFieldIterator<FProperty> PropIt(FunctionToCall); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
		{
			FProperty* FuncParamProperty = *PropIt;
			if (FuncParamProperty->HasAnyPropertyFlags(CPF_ReturnParm))
			{
				if (Statement.LHS)
				{
					EmitDestinationExpression(Statement.LHS);
				}
				bHasOutputValue = true;
			}
			else if (FuncParamProperty->HasAnyPropertyFlags(CPF_OutParm) && !FuncParamProperty->HasAnyPropertyFlags(CPF_ConstParm))
			{
				// Non const values passed by ref are also an output
				bHasOutputValue = true;
			}
		}

		const bool bFinalFunction = FunctionToCall->HasAnyFunctionFlags(FUNC_Final) || Statement.bIsParentContext;
		using namespace UE::ProjectUtilities;
		const bool bMathCall = bFinalFunction
			&& FunctionToCall->HasAllFunctionFlags(FUNC_Static|FUNC_Final|FUNC_Native)
			&& !FunctionToCall->HasAnyFunctionFlags(FUNC_NetFuncFlags|FUNC_BlueprintAuthorityOnly|FUNC_BlueprintCosmetic|FUNC_NetRequest|FUNC_NetResponse)
			&& !FunctionToCall->GetOuterUClass()->IsChildOf(UInterface::StaticClass())
			// If the function has wildcard parameters (e.g. UKismetArrayLibrary) we assume that it
			// is doing things with reflection data at runtime and will therefore benefit from
			// the safety of a 'context' based function call (e.g. if context is invalid or
			// 'bArrayContextFailed' we can skip the function call):
			&& !UEdGraphSchema_K2::HasWildcardParams(FunctionToCall)
			// Don't emit math calls if the target function may disapear on us at loadtime/runtime:
			&& FBuildTargetSet::GetCallerTargetsUnsupportedByCallee(ClassBeingBuilt, FunctionToCall) == FBuildTargetSet();

		const bool bLocalScriptFunction =
			!FunctionToCall->HasAnyFunctionFlags(FUNC_Native|FUNC_NetFuncFlags|FUNC_BlueprintAuthorityOnly|FUNC_BlueprintCosmetic|FUNC_NetRequest|FUNC_NetResponse);

		// Handle the function calling context if needed
		FContextEmitter CallContextWriter(*this);

		if (!bMathCall) // math call doesn't need context
		{
			// RValue property is used to clear value after Access Violation. See UObject::ProcessContextOpcod
			// If the property from LHS is used, then the retured property (with CPF_ReturnParm) is cleared. But properties returned by ref are not cleared.
			FProperty* RValueProperty = Statement.LHS ? Statement.LHS->AssociatedVarProperty : nullptr;
			CallContextWriter.TryStartContext(Statement.FunctionContext, /*bUnsafeToSkip=*/ bHasOutputValue, Statement.bIsInterfaceContext, RValueProperty);
		}

		// Emit the call type
		if (FunctionToCall->HasAnyFunctionFlags(FUNC_Delegate))
		{
			// @todo: Default delegate functions are no longer callable (and also now have mangled names.)  FindField will fail.
			check(false);
		}
		else if (bFinalFunction)
		{
			if (bMathCall)
			{
				Writer << EX_CallMath;
			}
			else if(bLocalScriptFunction)
			{
				Writer << EX_LocalFinalFunction;
			}
			else
			{
				Writer << EX_FinalFunction;
			}
			// The function to call doesn't have a native index
			Writer << FunctionToCall;
		}
		else
		{
			FName FunctionName(FunctionToCall->GetFName());
			if(bLocalScriptFunction)
			{
				Writer << EX_LocalVirtualFunction;
			}
			else
			{
				Writer << EX_VirtualFunction;
			}
			Writer << FunctionName;
		}

		const bool bIsCustomThunk = FunctionToCall->HasMetaData(FBlueprintMetadata::MD_CustomThunk);
		// Emit function parameters
		int32 NumParams = 0;
		for (TFieldIterator<FProperty> PropIt(FunctionToCall); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
		{
			FProperty* FuncParamProperty = *PropIt;

			if (!FuncParamProperty->HasAnyPropertyFlags(CPF_ReturnParm))
			{
				FBPTerminal* Term = Statement.RHS[NumParams];
				check(Term != NULL);

				// Latent function handling:  Need to emit a fixup request into the FLatentInfo struct
 				if (bIsUbergraph && FuncParamProperty->GetName() == FunctionToCall->GetMetaData(FBlueprintMetadata::MD_LatentInfo))
 				{
					EmitLatentInfoTerm(Term, FuncParamProperty, Statement.TargetLabel);
 				}
				else
				{
					// Native type of a wildcard parameter should be ignored.
					const bool bBadCoerceProperty = bIsCustomThunk && !Term->Type.IsContainer() && UEdGraphSchema_K2::IsWildcardProperty(FuncParamProperty);
					// When no coerce property is passed, a type of literal will be retrieved from the term.
					EmitTerm(Term, bBadCoerceProperty ? nullptr : FuncParamProperty);
				}
				NumParams++;
			}
		}

		const bool bIsVariadic = FunctionToCall->HasMetaData(FBlueprintMetadata::MD_Variadic);
		if (bIsVariadic)
		{
			// Variadic functions may have extra terms they need to emit after the main set of function arguments
			// These are all considered wildcards so no type checking will be performed on them
			for (; NumParams < Statement.RHS.Num(); ++NumParams)
			{
				FBPTerminal* Term = Statement.RHS[NumParams];
				check(Term);
				EmitTerm(Term, nullptr);
			}
		}

		// End of parameter list
		Writer << EX_EndFunctionParms;
	}

	void EmitCallDelegate(FBlueprintCompiledStatement& Statement)
	{
		UFunction* FunctionToCall = Statement.FunctionToCall;
		check(NULL != FunctionToCall);
		check(NULL != Statement.FunctionContext);
		check(FunctionToCall->HasAnyFunctionFlags(FUNC_Delegate));

		ClassBeingBuilt->CalledFunctions.Emplace(FunctionToCall);

		// The function to call doesn't have a native index
		Writer << EX_CallMulticastDelegate;
		Writer << FunctionToCall;
		EmitTerm(Statement.FunctionContext);

		// Emit function parameters
		int32 NumParams = 0;
		for (TFieldIterator<FProperty> PropIt(FunctionToCall); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
		{
			FProperty* FuncParamProperty = *PropIt;

			FBPTerminal* Term = Statement.RHS[NumParams];
			check(Term != NULL);

			// Emit parameter term normally
			EmitTerm(Term, FuncParamProperty);

			NumParams++;
		}

		// End of parameter list
		Writer << EX_EndFunctionParms;
	}

	void EmitTerm(FBPTerminal* Term, const FProperty* CoerceProperty = NULL, FBPTerminal* RValueTerm = NULL)
	{
		if (Term->InlineGeneratedParameter)
		{
			ensure(!Term->InlineGeneratedParameter->bIsJumpTarget);
			auto TermSourceAsNode = Cast<UEdGraphNode>(Term->Source);
			auto TermSourceAsPin = Term->SourcePin;
			UEdGraphNode* SourceNode = TermSourceAsNode ? TermSourceAsNode
				: (TermSourceAsPin ? TermSourceAsPin->GetOwningNodeUnchecked() : nullptr);
			if (ensure(CurrentCompilerContext && CurrentFunctionContext))
			{
				GenerateCodeForStatement(*CurrentCompilerContext, *CurrentFunctionContext, *Term->InlineGeneratedParameter, SourceNode);
			}
		}
		else if (Term->Context == NULL)
		{
			EmitTermExpr(Term, CoerceProperty);
		}
		else
		{
			if (Term->Context->IsStructContextType())
			{
				check(Term->AssociatedVarProperty);

				Writer << EX_StructMemberContext;
				Writer << Term->AssociatedVarProperty;

				// Now run the context expression
				EmitTerm(Term->Context, NULL);
			}
			else
			{
				// If this is the top of the chain this context, then save it off the r-value and pass it down the chain so we can safely handle runtime null contexts
				if( RValueTerm == NULL )
				{
					RValueTerm = Term;
				}

 				FContextEmitter CallContextWriter(*this);
				FProperty* RValueProperty = RValueTerm->AssociatedVarProperty;
				CallContextWriter.TryStartContext(Term->Context, /*@TODO: bUnsafeToSkip*/ true, /*bIsInterfaceContext*/ false, RValueProperty);

				EmitTermExpr(Term, CoerceProperty);
			}
		}
	}

	void EmitDestinationExpression(FBPTerminal* DestinationExpression)
	{
		check(Schema && DestinationExpression && !DestinationExpression->Type.PinCategory.IsNone());

		const bool bIsContainer = DestinationExpression->Type.IsContainer();
		const bool bIsDelegate = UEdGraphSchema_K2::PC_Delegate == DestinationExpression->Type.PinCategory;
		const bool bIsMulticastDelegate = UEdGraphSchema_K2::PC_MCDelegate == DestinationExpression->Type.PinCategory;
		const bool bIsBoolean = UEdGraphSchema_K2::PC_Boolean == DestinationExpression->Type.PinCategory;
		const bool bIsObj = (UEdGraphSchema_K2::PC_Object == DestinationExpression->Type.PinCategory) || (UEdGraphSchema_K2::PC_Class == DestinationExpression->Type.PinCategory);
		const bool bIsSoftObject = UEdGraphSchema_K2::PC_SoftObject == DestinationExpression->Type.PinCategory;
		const bool bIsWeakObjPtr = DestinationExpression->Type.bIsWeakPointer;

		if (bIsContainer)
		{
			Writer << EX_Let;
			ensure(DestinationExpression->AssociatedVarProperty);
			Writer << DestinationExpression->AssociatedVarProperty;
		}
		else if (bIsMulticastDelegate)
		{
			Writer << EX_LetMulticastDelegate;
		}
		else if (bIsDelegate)
		{
			Writer << EX_LetDelegate;
		}
		else if (bIsBoolean)
		{
			Writer << EX_LetBool;
		}
		else if (bIsObj && !bIsSoftObject)
		{
			if( !bIsWeakObjPtr )
			{
				Writer << EX_LetObj;
			}
			else
			{
				Writer << EX_LetWeakObjPtr;
			}
		}
		else
		{
			Writer << EX_Let;
			Writer << DestinationExpression->AssociatedVarProperty;
		}
		EmitTerm(DestinationExpression);
	}

	void EmitAssignmentStatment(FBlueprintCompiledStatement& Statement)
	{
		FBPTerminal* DestinationExpression = Statement.LHS;
		FBPTerminal* SourceExpression = Statement.RHS[0];

		EmitDestinationExpression(DestinationExpression);

		EmitTerm(SourceExpression, DestinationExpression->AssociatedVarProperty);
	}

	void EmitAssignmentOnPersistentFrameStatment(FBlueprintCompiledStatement& Statement)
	{
		FBPTerminal* DestinationExpression = Statement.LHS;
		FBPTerminal* SourceExpression = Statement.RHS[0];

		Writer << EX_LetValueOnPersistentFrame;
		check(ClassBeingBuilt && ClassBeingBuilt->UberGraphFunction);
		Writer << DestinationExpression->AssociatedVarProperty;

		EmitTerm(SourceExpression, DestinationExpression->AssociatedVarProperty);
	}

	void EmitCastObjToInterfaceStatement(FBlueprintCompiledStatement& Statement)
	{
		FBPTerminal* DestinationExpression = Statement.LHS;
		FBPTerminal* InterfaceExpression = Statement.RHS[0];
		FBPTerminal* TargetExpression = Statement.RHS[1];

		Writer << EX_Let;
		FProperty* PropertyToHandleComplexStruct = nullptr;
		Writer << PropertyToHandleComplexStruct;
		EmitTerm(DestinationExpression);

		Writer << EX_ObjToInterfaceCast;
		UClass* ClassPtr = CastChecked<UClass>(InterfaceExpression->ObjectLiteral);
		check(ClassPtr);
		Writer << ClassPtr;
		EmitTerm(TargetExpression, (FProperty*)(GetDefault<FObjectProperty>()));
	}

	void EmitCastBetweenInterfacesStatement(FBlueprintCompiledStatement& Statement)
	{
		FBPTerminal* DestinationExpression = Statement.LHS;
		FBPTerminal* InterfaceExpression   = Statement.RHS[0];
		FBPTerminal* TargetExpression      = Statement.RHS[1];

		Writer << EX_Let;
		FProperty* PropertyToHandleComplexStruct = nullptr;
		Writer << PropertyToHandleComplexStruct;
		EmitTerm(DestinationExpression);

		Writer << EX_CrossInterfaceCast;
		UClass* ClassPtr = CastChecked<UClass>(InterfaceExpression->ObjectLiteral);
		check(ClassPtr);
		Writer << ClassPtr;
		EmitTerm(TargetExpression, (FProperty*)(GetDefault<FInterfaceProperty>()));
	}

	void EmitCastInterfaceToObjStatement(FBlueprintCompiledStatement& Statement)
	{
		FBPTerminal* DestinationExpression     = Statement.LHS;
		FBPTerminal* ResultObjClassExpression  = Statement.RHS[0];
		FBPTerminal* TargetInterfaceExpression = Statement.RHS[1];

		Writer << EX_Let;
		FProperty* PropertyToHandleComplexStruct = nullptr;
		Writer << PropertyToHandleComplexStruct;
		EmitTerm(DestinationExpression);

		Writer << EX_InterfaceToObjCast;
		UClass* ClassPtr = CastChecked<UClass>(ResultObjClassExpression->ObjectLiteral);
		check(ClassPtr != nullptr);
		Writer << ClassPtr;
		EmitTerm(TargetInterfaceExpression, (FProperty*)(GetDefault<FObjectProperty>()));
	}

	void EmitDynamicCastStatement(FBlueprintCompiledStatement& Statement)
	{
		FBPTerminal* DestinationExpression = Statement.LHS;
		FBPTerminal* ResultClassExpression = Statement.RHS[0];
		FBPTerminal* TargetExpression = Statement.RHS[1];

		Writer << EX_Let;
		FProperty* PropertyToHandleComplexStruct = nullptr;
		Writer << PropertyToHandleComplexStruct;
		EmitTerm(DestinationExpression);

		Writer << EX_DynamicCast;
		UClass* ClassPtr = CastChecked<UClass>(ResultClassExpression->ObjectLiteral);
		Writer << ClassPtr;
		EmitTerm(TargetExpression, (FProperty*)(GetDefault<FObjectProperty>()));
	}

	void EmitMetaCastStatement(FBlueprintCompiledStatement& Statement)
	{
		FBPTerminal* DestinationExpression = Statement.LHS;
		FBPTerminal* InterfaceExpression = Statement.RHS[0];
		FBPTerminal* TargetExpression = Statement.RHS[1];

		Writer << EX_Let;
		FProperty* PropertyToHandleComplexStruct = nullptr;
		Writer << PropertyToHandleComplexStruct;
		EmitTerm(DestinationExpression);

		Writer << EX_MetaCast;
		UClass* ClassPtr = CastChecked<UClass>(InterfaceExpression->ObjectLiteral);
		Writer << ClassPtr;
		EmitTerm(TargetExpression, (FProperty*)(GetDefault<FClassProperty>()));
	}

	void EmitObjectToBoolStatement(FBlueprintCompiledStatement& Statement)
	{
		FBPTerminal* DestinationExpression = Statement.LHS;
		FBPTerminal* TargetExpression = Statement.RHS[0];

		UClass* PSCObjClass = Cast<UClass>(TargetExpression->Type.PinSubCategoryObject.Get());
		const bool bIsInterfaceCast = (PSCObjClass && PSCObjClass->HasAnyClassFlags(CLASS_Interface));

		Writer << EX_Let;
		FProperty* PropertyToHandleComplexStruct = nullptr;
		Writer << PropertyToHandleComplexStruct;
		EmitTerm(DestinationExpression);

		Writer << EX_Cast;
		ECastToken CastToken = !bIsInterfaceCast ? CST_ObjectToBool : CST_InterfaceToBool;
		uint8 CastType = static_cast<uint8>(CastToken);
		Writer << CastType;

		FProperty* TargetProperty = !bIsInterfaceCast ? ((FProperty*)(GetDefault<FObjectProperty>())) : ((FProperty*)(GetDefault<FInterfaceProperty>()));
		EmitTerm(TargetExpression, TargetProperty);
	}

	void EmitAddMulticastDelegateStatement(FBlueprintCompiledStatement& Statement)
	{
		FBPTerminal* Delegate = Statement.LHS;
		FBPTerminal* DelegateToAdd = Statement.RHS[0];

		Writer << EX_AddMulticastDelegate;
		EmitTerm(Delegate);
		EmitTerm(DelegateToAdd);
	}

	void EmitRemoveMulticastDelegateStatement(FBlueprintCompiledStatement& Statement)
	{
		FBPTerminal* Delegate = Statement.LHS;
		FBPTerminal* DelegateToAdd = Statement.RHS[0];

		Writer << EX_RemoveMulticastDelegate;
		EmitTerm(Delegate);
		EmitTerm(DelegateToAdd);
	}

	void EmitBindDelegateStatement(FBlueprintCompiledStatement& Statement)
	{
		check(2 == Statement.RHS.Num());
		FBPTerminal* Delegate = Statement.LHS;
		FBPTerminal* NameTerm = Statement.RHS[0];
		FBPTerminal* ObjectTerm = Statement.RHS[1];
		check(Delegate && ObjectTerm);
		check(NameTerm && NameTerm->bIsLiteral);
		check(!NameTerm->Name.IsEmpty());

		FName FunctionName(*(NameTerm->Name));
		Writer << EX_BindDelegate;
		Writer << FunctionName;

		EmitTerm(Delegate);
		EmitTerm(ObjectTerm, (FProperty*)(GetDefault<FObjectProperty>()));
	}

	void EmitClearMulticastDelegateStatement(FBlueprintCompiledStatement& Statement)
	{
		FBPTerminal* Delegate = Statement.LHS;

		Writer << EX_ClearMulticastDelegate;
		EmitTerm(Delegate);
	}

	void EmitCreateArrayStatement(FBlueprintCompiledStatement& Statement)
	{
		Writer << EX_SetArray;

		FBPTerminal* ArrayTerm = Statement.LHS;
		EmitTerm(ArrayTerm);

		FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(ArrayTerm->AssociatedVarProperty);
		FProperty* InnerProperty = ArrayProperty->Inner;

		for(auto ArrayItemIt = Statement.RHS.CreateIterator(); ArrayItemIt; ++ArrayItemIt)
		{
			FBPTerminal* Item = *ArrayItemIt;
			EmitTerm(Item, (Item->bIsLiteral ? InnerProperty : NULL));
		}

		Writer << EX_EndArray;
	}

	void EmitCreateSetStatement(FBlueprintCompiledStatement& Statement)
	{
		Writer << EX_SetSet;

		FBPTerminal* SetTerm = Statement.LHS;
		EmitTerm(SetTerm);
		int32 ElementNum = Statement.RHS.Num();

		Writer << ElementNum; // number of elements in the set, used for reserve call

		FSetProperty* SetProperty = CastFieldChecked<FSetProperty>(SetTerm->AssociatedVarProperty);
		FProperty* InnerProperty = SetProperty->ElementProp;

		for(FBPTerminal* Item : Statement.RHS)
		{
			EmitTerm(Item, (Item->bIsLiteral ? InnerProperty : NULL));
		}

		Writer << EX_EndSet;
	}

	void EmitCreateMapStatement(FBlueprintCompiledStatement& Statement)
	{
		Writer << EX_SetMap;

		FBPTerminal* MapTerm = Statement.LHS;
		EmitTerm(MapTerm);

		ensureMsgf(Statement.RHS.Num() % 2 == 0, TEXT("Expected even number of key/values whe emitting map statement"));
		int32 ElementNum = Statement.RHS.Num() / 2;

		Writer << ElementNum;

		FMapProperty* MapProperty = CastFieldChecked<FMapProperty>(MapTerm->AssociatedVarProperty);

		for(auto MapItemIt = Statement.RHS.CreateIterator(); MapItemIt; ++MapItemIt)
		{
			FBPTerminal* Item = *MapItemIt;
			EmitTerm(Item, (Item->bIsLiteral ? MapProperty->KeyProp : NULL));
			++MapItemIt;
			Item = *MapItemIt;
			EmitTerm(Item, (Item->bIsLiteral ? MapProperty->ValueProp : NULL));
		}

		Writer << EX_EndMap;
	}

	void EmitGoto(FBlueprintCompiledStatement& Statement)
	{
		if (Statement.Type == KCST_ComputedGoto)
		{
			// Emit the computed jump operation
			Writer << EX_ComputedJump;

			// Now include the integer offset expression
			EmitTerm(Statement.LHS, (FProperty*)(GetDefault<FIntProperty>()));
		}
		else if (Statement.Type == KCST_GotoIfNot)
		{
			// Emit the jump with a dummy address
			Writer << EX_JumpIfNot;
			CodeSkipSizeType PatchUpNeededAtOffset = Writer.EmitPlaceholderSkip();

			// Queue up a fixup to be done once all label offsets are known
			JumpTargetFixupMap.Add(PatchUpNeededAtOffset, FCodeSkipInfo(FCodeSkipInfo::Fixup, Statement.TargetLabel));

			// Now include the boolean expression
			EmitTerm(Statement.LHS, (FProperty*)(GetDefault<FBoolProperty>()));
		}
		else if (Statement.Type == KCST_EndOfThreadIfNot)
		{
			// Emit the pop if not opcode
			Writer << EX_PopExecutionFlowIfNot;

			// Now include the boolean expression
			EmitTerm(Statement.LHS, (FProperty*)(GetDefault<FBoolProperty>()));
		}
		else if (Statement.Type == KCST_UnconditionalGoto)
		{
			// Emit the jump with a dummy address
			Writer << EX_Jump;
			CodeSkipSizeType PatchUpNeededAtOffset = Writer.EmitPlaceholderSkip();

			// Queue up a fixup to be done once all label offsets are known
			JumpTargetFixupMap.Add(PatchUpNeededAtOffset, FCodeSkipInfo(FCodeSkipInfo::Fixup, Statement.TargetLabel));
		}
		else if (Statement.Type == KCST_GotoReturn)
		{
			// Emit the jump with a dummy address
			Writer << EX_Jump;
			CodeSkipSizeType PatchUpNeededAtOffset = Writer.EmitPlaceholderSkip();

			// Queue up a fixup to be done once all label offsets are known
			JumpTargetFixupMap.Add(PatchUpNeededAtOffset, FCodeSkipInfo(FCodeSkipInfo::Fixup, &ReturnStatement));
		}
		else if (Statement.Type == KCST_GotoReturnIfNot)
		{
			// Emit the jump with a dummy address
			Writer << EX_JumpIfNot;
			CodeSkipSizeType PatchUpNeededAtOffset = Writer.EmitPlaceholderSkip();

			// Queue up a fixup to be done once all label offsets are known
			JumpTargetFixupMap.Add(PatchUpNeededAtOffset, FCodeSkipInfo(FCodeSkipInfo::Fixup, &ReturnStatement));

			// Now include the boolean expression
			EmitTerm(Statement.LHS, (FProperty*)(GetDefault<FBoolProperty>()));
		}
		else
		{
			ensureMsgf(false, TEXT("FScriptBuilderBase::EmitGoto unknown type"));
		}
	}

	void EmitPushExecState(FBlueprintCompiledStatement& Statement)
	{
		// Push the address onto the flow stack
		Writer << EX_PushExecutionFlow;
		CodeSkipSizeType PatchUpNeededAtOffset = Writer.EmitPlaceholderSkip();

		// Mark the target for fixup once the addresses have been resolved
		JumpTargetFixupMap.Add(PatchUpNeededAtOffset, FCodeSkipInfo(FCodeSkipInfo::Fixup, Statement.TargetLabel));
	}

	void EmitPopExecState(FBlueprintCompiledStatement& Statement)
	{
		// Pop the state off the flow stack
		Writer << EX_PopExecutionFlow;
	}

	void EmitReturn(FKismetFunctionContext& Context)
	{
		FProperty* ReturnProperty = Context.Function->GetReturnProperty();

		Writer << EX_Return;

		if (ReturnProperty == NULL)
		{
			Writer << EX_Nothing;
		}
		else
		{
			Writer << EX_LocalOutVariable;
			Writer << ReturnProperty;
		}
	}

	void EmitSwitchValue(FBlueprintCompiledStatement& Statement)
	{
		const int32 TermsBeforeCases = 1;
		const int32 TermsPerCase = 2;

		if ((Statement.RHS.Num() < 4) || (1 == (Statement.RHS.Num() % 2)))
		{
			// Error
			ensure(false);
		}

		Writer << EX_SwitchValue;
		// number of cases (without default)
		uint16 NumCases = IntCastChecked<uint16, int32>((Statement.RHS.Num() - 2) / TermsPerCase);
		Writer << NumCases;
		// end goto index
		CodeSkipSizeType PatchUpNeededAtOffset = Writer.EmitPlaceholderSkip();

		// index term
		auto IndexTerm = Statement.RHS[0];
		check(IndexTerm);
		EmitTerm(IndexTerm);
		FProperty* VirtualIndexProperty = IndexTerm->AssociatedVarProperty;
		check(VirtualIndexProperty);

		auto DefaultTerm = Statement.RHS[TermsBeforeCases + NumCases*TermsPerCase];
		check(DefaultTerm);
		FProperty* VirtualValueProperty = DefaultTerm->AssociatedVarProperty;
		check(VirtualValueProperty);

		for (uint16 TermIndex = TermsBeforeCases; TermIndex < (NumCases * TermsPerCase); ++TermIndex)
		{
			EmitTerm(Statement.RHS[TermIndex], VirtualIndexProperty); // it's a literal value
			++TermIndex;
			CodeSkipSizeType PatchOffsetToNextCase = Writer.EmitPlaceholderSkip();
			EmitTerm(Statement.RHS[TermIndex], VirtualValueProperty);  // it could be literal for 'self'
			Writer.CommitSkip(PatchOffsetToNextCase, Writer.ScriptBuffer.Num());
		}

		// default term
		EmitTerm(DefaultTerm);

		Writer.CommitSkip(PatchUpNeededAtOffset, Writer.ScriptBuffer.Num());
	}

	void EmitInstrumentation(FKismetCompilerContext& CompilerContext, FKismetFunctionContext& FunctionContext, FBlueprintCompiledStatement& Statement, UEdGraphNode* SourceNode)
	{
		// Allows us to turn off instrumentation to clean up the disassembly of functions for comparing the output of different USVM bytecode compilers
		static const FBoolConfigValueHelper bShouldSuppressInstrumentation(TEXT("Kismet"), TEXT("bSuppressInstrumentation"), GEngineIni);
		if (bShouldSuppressInstrumentation)
		{
			return;
		}

		int32 Offset = Writer.ScriptBuffer.Num();

		if (Statement.Type == KCST_DebugSite)
		{
			Writer << EX_Tracepoint;
		}
		else if (Statement.Type == KCST_WireTraceSite)
		{
			Writer << EX_WireTracepoint;
		}
		else
		{
			uint8 EventType = 0;
			switch (Statement.Type)
			{
			case KCST_InstrumentedEvent:				EventType = EScriptInstrumentation::InlineEvent; break;
			case KCST_InstrumentedEventStop:			EventType = EScriptInstrumentation::Stop; break;
			case KCST_InstrumentedWireExit:				EventType = EScriptInstrumentation::NodeExit; break;
			case KCST_InstrumentedWireEntry:			EventType = EScriptInstrumentation::NodeEntry; break;
			case KCST_InstrumentedPureNodeEntry:		EventType = EScriptInstrumentation::PureNodeEntry; break;
			case KCST_InstrumentedStatePush:			EventType = EScriptInstrumentation::PushState; break;
			case KCST_InstrumentedStateRestore:			EventType = EScriptInstrumentation::RestoreState; break;
			case KCST_InstrumentedStateReset:			EventType = EScriptInstrumentation::ResetState; break;
			case KCST_InstrumentedStateSuspend:			EventType = EScriptInstrumentation::SuspendState; break;
			case KCST_InstrumentedStatePop:				EventType = EScriptInstrumentation::PopState; break;
			case KCST_InstrumentedTunnelEndOfThread:	EventType = EScriptInstrumentation::TunnelEndOfThread; break;
			}
			Writer << EX_InstrumentationEvent;
			Writer << EventType;
			if (EventType == EScriptInstrumentation::InlineEvent)
			{
				FName EventName(*Statement.Comment);
				Writer << EventName;
			}
			else if (EventType == EScriptInstrumentation::SuspendState)
			{
				if (Statement.TargetLabel->TargetLabel)
				{
					CodeSkipSizeType PatchUpNeededAtOffset = Writer.EmitPlaceholderSkip();
					FCodeSkipInfo CodeSkipInfo(FCodeSkipInfo::InstrumentedDelegateFixup, Statement.TargetLabel->TargetLabel, &Statement);
					if (Statement.TargetLabel->FunctionToCall)
					{
						CodeSkipInfo.DelegateName = Statement.TargetLabel->FunctionToCall->GetFName();
					}
					// Queue up a fixup to be done once all label offsets are known
					JumpTargetFixupMap.Add(PatchUpNeededAtOffset, CodeSkipInfo);
				}
			}
		}

		TArray<UEdGraphPin*> PinContextArray(Statement.PureOutputContextArray);
		if (Statement.ExecContext != nullptr)
		{
			PinContextArray.Add(Statement.ExecContext);
		}

		for (auto PinContext : PinContextArray)
		{
			UEdGraphPin const* TrueSourcePin = FunctionContext.MessageLog.FindSourcePin(PinContext);
			// Source pin can be marked as pending kill if it was a generated pin that node logic decided to disown, e.g.
			// logic in UK2Node_CallFunction to handle bWantsEnumToExecExpansion:
			if (TrueSourcePin && !TrueSourcePin->IsPendingKill())
			{
				ClassBeingBuilt->GetDebugData().RegisterPinToCodeAssociation(TrueSourcePin, FunctionContext.Function, Offset);
			}
		}

		if (SourceNode != NULL)
		{
			// Record where this NOP is
			UEdGraphNode* TrueSourceNode = Cast<UEdGraphNode>(FunctionContext.MessageLog.FindSourceObject(SourceNode));
			if (TrueSourceNode)
			{
				const bool bInstrumentedBreakpoint = Statement.Type == KCST_InstrumentedWireEntry;
				const bool bBreakpointSite = Statement.Type == KCST_DebugSite || bInstrumentedBreakpoint;

				// If the intermediate node was the result of a tunnel expansion (e.g. macro/composite), gather the chain of tunnel instance
				// source nodes that were expanded to eventually include the node. This information is used to construct a lookup table that's
				// used to help determine whether or not the current instruction pointer maps back to a tunnel instance node in a source graph.
				// For example:
				//
				//	A (Macro instance node in top-level source graph)
				//	|
				//	+- [...expansion of A...] + B (Composite node in A's macro source graph)
				//	   |
				//	   +- [...expansion of B...] + C (Macro instance node in B's collapsed child subgraph)
				//	      |
				//        +- [...expansion of C...]
				//
				//  The intermediate exec nodes in each expansion set will map back to C through MessageLog.GetIntermediateTunnelInstance().
				//	Thus, if SourceNode was created as the result of a tunnel instance expansion, it will map back to an outer Tunnel Instance
				//	node. If the Tunnel Instance node itself was created as the result of a tunnel instance expansion, it will map back to
				//	another outer Tunnel Instance node as well. This will continue until we run out of Tunnel Instance nodes. The set of Tunnel
				//	Instance nodes that we find constitutes the full expansion hierarchy. We then map the hierarchy back to their matching
				//	source nodes and register the set as the Tunnel Instance node chain that's associated with the current instruction offset.
				//
				TArray<TWeakObjectPtr<UEdGraphNode>> ExpansionSourceNodes;
				if (const UEdGraphNode* OuterTunnelInstance = FunctionContext.MessageLog.GetIntermediateTunnelInstance(SourceNode))
				{
					do
					{
						// Map the intermediate tunnel instance node back to its original source and add it to the array.
						if (const UEdGraphNode* ExpansionSourceNode = Cast<UEdGraphNode>(FunctionContext.MessageLog.FindSourceObject(OuterTunnelInstance)))
						{
							ExpansionSourceNodes.Add(MakeWeakObjectPtr(const_cast<UEdGraphNode*>(ExpansionSourceNode)));
						}

						// Continue back up the chain until we run out of expansion source nodes (this ensures that we include any nested expansions).
						OuterTunnelInstance = FunctionContext.MessageLog.GetIntermediateTunnelInstance(OuterTunnelInstance);

					} while (OuterTunnelInstance);
				}

				// Register the debug information for the node.
				ClassBeingBuilt->GetDebugData().RegisterNodeToCodeAssociation(TrueSourceNode, ExpansionSourceNodes, FunctionContext.Function, Offset, bBreakpointSite);

				// Track pure node script code range for the current impure (exec) node
				if (Statement.Type == KCST_InstrumentedPureNodeEntry)
				{
					if (PureNodeEntryCount == 0)
					{
						// Indicates the starting offset for this pure node call chain.
						PureNodeEntryStart = Offset;
					}

					++PureNodeEntryCount;
				}
				else if (Statement.Type == KCST_InstrumentedWireEntry && PureNodeEntryCount > 0)
				{
					// Map script code range for the full set of pure node inputs feeding in to the current impure (exec) node at the current offset
					ClassBeingBuilt->GetDebugData().RegisterPureNodeScriptCodeRange(TrueSourceNode, FunctionContext.Function, FInt32Range(PureNodeEntryStart, Offset));

					// Reset pure node code range tracking.
					PureNodeEntryCount = 0;
					PureNodeEntryStart = 0;
				}
			}
		}
	}

	void EmitArrayGetByRef(FBlueprintCompiledStatement& Statement)
	{
		Writer << EX_ArrayGetByRef;
		// The array variable
		EmitTerm(Statement.RHS[0]);
		// The index to access in the array
		EmitTerm(Statement.RHS[1], (FProperty*)(GetDefault<FIntProperty>()));
	}

	void EmitCastStatement(FBlueprintCompiledStatement& Statement)
	{
		FBPTerminal* DestinationExpression = Statement.LHS;
		FBPTerminal* TargetExpression = Statement.RHS[0];

		Writer << EX_Let;
		FProperty* PropertyToHandleComplexStruct = nullptr;
		Writer << PropertyToHandleComplexStruct;
		EmitTerm(DestinationExpression);

		Writer << EX_Cast;

		ECastToken CastType = CST_Max;

		switch (Statement.Type)
		{
			case KCST_DoubleToFloatCast:
				CastType = CST_DoubleToFloat;
				break;
			case KCST_FloatToDoubleCast:
				CastType = CST_FloatToDouble;
				break;
			default:
				check(false);
				break;
		}

		Writer << CastType;

		EmitTerm(TargetExpression);
	}

	void PushReturnAddress(FBlueprintCompiledStatement& ReturnTarget)
	{
		Writer << EX_PushExecutionFlow;
		CodeSkipSizeType PatchUpNeededAtOffset = Writer.EmitPlaceholderSkip();

		JumpTargetFixupMap.Add(PatchUpNeededAtOffset, FCodeSkipInfo(FCodeSkipInfo::Fixup, &ReturnTarget));
	}

	void CloseScript()
	{
		Writer << EX_EndOfScript;
	}

	virtual ~FScriptBuilderBase()
	{
	}

	void GenerateCodeForStatement(FKismetCompilerContext& CompilerContext, FKismetFunctionContext& FunctionContext, FBlueprintCompiledStatement& Statement, UEdGraphNode* SourceNode)
	{
		TGuardValue<FKismetCompilerContext*> CompilerContextGuard(CurrentCompilerContext, &CompilerContext);
		TGuardValue<FKismetFunctionContext*> FunctionContextGuard(CurrentFunctionContext, &FunctionContext);

		// Record the start of this statement in the bytecode if it's needed as a target label
		if (Statement.bIsJumpTarget)
		{
			StatementLabelMap.Add(&Statement, Writer.ScriptBuffer.Num());
		}

		// Generate bytecode for the statement
		switch (Statement.Type)
		{
		case KCST_Nop:
			Writer << EX_Nothing;
			break;
		case KCST_CallFunction:
			EmitFunctionCall(CompilerContext, FunctionContext, Statement, SourceNode);
			break;
		case KCST_CallDelegate:
			EmitCallDelegate(Statement);
			break;
		case KCST_Assignment:
			EmitAssignmentStatment(Statement);
			break;
		case KCST_AssignmentOnPersistentFrame:
			EmitAssignmentOnPersistentFrameStatment(Statement);
			break;
		case KCST_CastObjToInterface:
			EmitCastObjToInterfaceStatement(Statement);
			break;
		case KCST_CrossInterfaceCast:
			EmitCastBetweenInterfacesStatement(Statement);
			break;
		case KCST_CastInterfaceToObj:
			EmitCastInterfaceToObjStatement(Statement);
			break;
		case KCST_DynamicCast:
			EmitDynamicCastStatement(Statement);
			break;
		case KCST_MetaCast:
			EmitMetaCastStatement(Statement);
			break;
		case KCST_ObjectToBool:
			EmitObjectToBoolStatement(Statement);
			break;
		case KCST_AddMulticastDelegate:
			EmitAddMulticastDelegateStatement(Statement);
			break;
		case KCST_RemoveMulticastDelegate:
			EmitRemoveMulticastDelegateStatement(Statement);
			break;
		case KCST_BindDelegate:
			EmitBindDelegateStatement(Statement);
			break;
		case KCST_ClearMulticastDelegate:
			EmitClearMulticastDelegateStatement(Statement);
			break;
		case KCST_CreateArray:
			EmitCreateArrayStatement(Statement);
			break;
		case KCST_ComputedGoto:
		case KCST_UnconditionalGoto:
		case KCST_GotoIfNot:
		case KCST_EndOfThreadIfNot:
		case KCST_GotoReturn:
		case KCST_GotoReturnIfNot:
			EmitGoto(Statement);
			break;
		case KCST_PushState:
			EmitPushExecState(Statement);
			break;
		case KCST_EndOfThread:
			EmitPopExecState(Statement);
			break;
		case KCST_Comment:
			// VM ignores comments
			break;
		case KCST_Return:
			EmitReturn(FunctionContext);
			break;
		case KCST_SwitchValue:
			EmitSwitchValue(Statement);
			break;
		case KCST_DebugSite:
		case KCST_WireTraceSite:
		case KCST_InstrumentedEvent:
		case KCST_InstrumentedEventStop:
		case KCST_InstrumentedWireEntry:
		case KCST_InstrumentedWireExit:
		case KCST_InstrumentedStatePush:
		case KCST_InstrumentedStateReset:
		case KCST_InstrumentedStateSuspend:
		case KCST_InstrumentedStatePop:
		case KCST_InstrumentedStateRestore:
		case KCST_InstrumentedPureNodeEntry:
		case KCST_InstrumentedTunnelEndOfThread:
			EmitInstrumentation(CompilerContext, FunctionContext, Statement, SourceNode);
			break;
		case KCST_ArrayGetByRef:
			EmitArrayGetByRef(Statement);
			break;
		case KCST_CreateSet:
			EmitCreateSetStatement(Statement);
			break;
		case KCST_CreateMap:
			EmitCreateMapStatement(Statement);
			break;
		case KCST_DoubleToFloatCast:
		case KCST_FloatToDoubleCast:
			EmitCastStatement(Statement);
			break;
		default:
			UE_LOGF(LogK2Compiler, Warning, "VM backend encountered unsupported statement type %d", (int32)Statement.Type);
		}
	}

	// Fix up all jump targets
	void PerformFixups()
	{
		for (TMap<CodeSkipSizeType, FCodeSkipInfo>::TIterator It(JumpTargetFixupMap); It; ++It)
		{
			CodeSkipSizeType OffsetToFix = It.Key();
			FCodeSkipInfo& CodeSkipInfo = It.Value();

			CodeSkipSizeType TargetStatementOffset = StatementLabelMap.FindChecked(CodeSkipInfo.TargetLabel);

			Writer.CommitSkip(OffsetToFix, TargetStatementOffset);

			if (CodeSkipInfo.Type == FCodeSkipInfo::InstrumentedDelegateFixup)
			{
				// Register delegate entrypoint offsets
				ClassBeingBuilt->GetDebugData().RegisterEntryPoint(TargetStatementOffset, CodeSkipInfo.DelegateName);
			}
		}

		JumpTargetFixupMap.Empty();
	}
};

//////////////////////////////////////////////////////////////////////////
// FKismetCompilerVMBackend

void FKismetCompilerVMBackend::GenerateCodeFromClass(UClass* SourceClass, TIndirectArray<FKismetFunctionContext>& Functions, bool bGenerateStubsOnly)
{
	// Generate script bytecode
	for (int32 i = 0; i < Functions.Num(); ++i)
	{
		FKismetFunctionContext& Function = Functions[i];
		if (Function.IsValid())
		{
			const bool bIsUbergraph = (i == 0);
			ConstructFunction(Function, bIsUbergraph, bGenerateStubsOnly);
		}
	}

	// Remove duplicates from CalledFunctions:
	UBlueprintGeneratedClass* ClassBeingBuilt = CastChecked<UBlueprintGeneratedClass>(SourceClass);
	TSet<TObjectPtr<UFunction>> Unique(ClassBeingBuilt->CalledFunctions);
	ClassBeingBuilt->CalledFunctions = Unique.Array();
}

void FKismetCompilerVMBackend::ConstructFunction(FKismetFunctionContext& FunctionContext, bool bIsUbergraph, bool bGenerateStubOnly)
{
	UFunction* Function = FunctionContext.Function;
	UBlueprintGeneratedClass* Class = FunctionContext.NewClass;

	FString FunctionName;
	Function->GetName(FunctionName);

	TArray<uint8>& ScriptArray = Function->Script;

	// Return statement, to push on FlowStack or to use with _GotoReturn
	FBlueprintCompiledStatement ReturnStatement;
	ReturnStatement.Type = KCST_Return;

	FScriptBuilderBase ScriptWriter(ScriptArray, Class, Schema, UbergraphStatementLabelMap, bIsUbergraph, ReturnStatement);

	if (!bGenerateStubOnly)
	{
		ReturnStatement.bIsJumpTarget = true;
		if (FunctionContext.bUseFlowStack)
		{
			ScriptWriter.PushReturnAddress(ReturnStatement);
		}

		// Emit code in the order specified by the linear execution list (the first node is always the entry point for the function)
		for (int32 NodeIndex = 0; NodeIndex < FunctionContext.LinearExecutionList.Num(); ++NodeIndex)
		{
			UEdGraphNode* StatementNode = FunctionContext.LinearExecutionList[NodeIndex];
			TArray<FBlueprintCompiledStatement*>* StatementList = FunctionContext.StatementsPerNode.Find(StatementNode);

			if (StatementList != nullptr)
			{
				for (int32 StatementIndex = 0; StatementIndex < StatementList->Num(); ++StatementIndex)
				{
					FBlueprintCompiledStatement* Statement = (*StatementList)[StatementIndex];

					ScriptWriter.GenerateCodeForStatement(CompilerContext, FunctionContext, *Statement, StatementNode);

					// Abort code generation on error (no need to process additional statements).
					if (FunctionContext.MessageLog.NumErrors > 0)
					{
						break;
					}
				}
			}

			// Reduce to a stub if any errors were raised. This ensures the VM won't attempt to evaluate an incomplete expression.
			if (FunctionContext.MessageLog.NumErrors > 0)
			{
				ScriptArray.Empty();
				ReturnStatement.bIsJumpTarget = false;
				break;
			}
		}
	}

	// Handle the function return value
	ScriptWriter.GenerateCodeForStatement(CompilerContext, FunctionContext, ReturnStatement, nullptr);

	// Fix up jump addresses
	ScriptWriter.PerformFixups();

	// Close out the script
	ScriptWriter.CloseScript();

	// Save off the offsets within the ubergraph, needed to patch up the stubs later on
	if (bIsUbergraph)
	{
		ScriptWriter.CopyStatementMapToUbergraphMap();
	}

	// Make sure we didn't overflow the maximum bytecode size
#if SCRIPT_LIMIT_BYTECODE_TO_64KB
	if (ScriptArray.Num() > 0xFFFF)
	{
		MessageLog.Error(TEXT("Script exceeded bytecode length limit of 64 KB"));
		ScriptArray.Empty();
		ScriptArray.Add(EX_EndOfScript);
	}
#else
	static_assert(sizeof(CodeSkipSizeType) == 4, "Update this code as size changed.");
#endif
}

#undef LOCTEXT_NAMESPACE
