// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserDefinedStructureCompilerUtils.h"

#include "Algo/Copy.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EdMode.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Serialization/ArchiveReplaceObjectAndStructPropertyRef.h"
#include "StructUtils/PropertyBag.h"
#include "StructUtils/UserDefinedStruct.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Kismet2/StructureEditorUtils.h"
#include "KismetCompiler.h"
#include "KismetCompilerMisc.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Guid.h"
#include "StructUtils/StructReinstancer.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Templates/UnrealTemplate.h"
#include "Tools/LegacyEdModeWidgetHelpers.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/FieldIterator.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "UObject/ReferencerFinder.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "UserDefinedStructure/UserDefinedStructEditorData.h"

#define LOCTEXT_NAMESPACE "StructureCompiler"

struct FUserDefinedStructureCompilerInner
{
	struct FBlueprintUserStructData
	{
		TArray<uint8> SkeletonCDOData;
		TArray<uint8> GeneratedCDOData;
	};

	static void ClearStructReferencesInBP(UBlueprint* FoundBlueprint, TMap<UBlueprint*, FBlueprintUserStructData>& BlueprintsToRecompile)
	{
		if (!BlueprintsToRecompile.Contains(FoundBlueprint))
		{
			FBlueprintUserStructData& BlueprintData = BlueprintsToRecompile.Add(FoundBlueprint);

			// Write CDO data to temp archive
			//FObjectWriter SkeletonMemoryWriter(FoundBlueprint->SkeletonGeneratedClass->GetDefaultObject(), BlueprintData.SkeletonCDOData);
			//FObjectWriter MemoryWriter(FoundBlueprint->GeneratedClass->GetDefaultObject(), BlueprintData.GeneratedCDOData);

			for (UFunction* Function : TFieldRange<UFunction>(FoundBlueprint->GeneratedClass, EFieldIteratorFlags::ExcludeSuper))
			{
				Function->Script.Empty();
			}
			FoundBlueprint->Status = BS_Dirty;
		}
	}

	static TNotNull<UUserDefinedStruct*> CreateDuplicate(TNotNull<UUserDefinedStruct*> StructureToReinstance)
	{
		UUserDefinedStruct* DuplicatedStruct = nullptr;
		{
			const FString ReinstancedName = FString::Printf(TEXT("STRUCT_REINST_%s"), *StructureToReinstance->GetName());
			const FName UniqueName = MakeUniqueObjectName(GetTransientPackage(), UUserDefinedStruct::StaticClass(), FName(*ReinstancedName));

			TGuardValue<FIsDuplicatingClassForReinstancing, bool> IsDuplicatingClassForReinstancing(GIsDuplicatingClassForReinstancing, true);
			DuplicatedStruct = (UUserDefinedStruct*)StaticDuplicateObject(StructureToReinstance, GetTransientPackage(), UniqueName, ~RF_Transactional);
		}

		DuplicatedStruct->Guid = StructureToReinstance->Guid;
		DuplicatedStruct->Bind();
		DuplicatedStruct->StaticLink(true);
		DuplicatedStruct->PrimaryStruct = StructureToReinstance;
		DuplicatedStruct->Status = EUserDefinedStructureStatus::UDSS_Duplicate;
		DuplicatedStruct->SetFlags(RF_Transient);

		CastChecked<UUserDefinedStructEditorData>(DuplicatedStruct->EditorData)->RecreateDefaultInstance();

		return DuplicatedStruct;
	}

	static void ReplaceStructWithTempDuplicate(
		TNotNull<UScriptStruct*> StructureToReinstance,
		TNotNull<UScriptStruct*> DuplicatedStruct,
		TMap<UBlueprint*, FBlueprintUserStructData>& BlueprintsToRecompile,
		TArray<UScriptStruct*>& ChangedStructs)
	{
		// List of unique classes and structs to regenerate bytecode and property referenced objects list
		TSet<UStruct*> StructsToRegenerateReferencesFor;

		for (TAllFieldsIterator<FStructProperty> FieldIt(RF_NoFlags, EInternalObjectFlags::Garbage); FieldIt; ++FieldIt)
		{
			FStructProperty* StructProperty = *FieldIt;
			if (StructProperty && (StructureToReinstance == StructProperty->Struct))
			{
				if (UBlueprintGeneratedClass* OwnerClass = Cast<UBlueprintGeneratedClass>(StructProperty->GetOwnerClass()))
				{
					if (UBlueprint* FoundBlueprint = Cast<UBlueprint>(OwnerClass->ClassGeneratedBy))
					{
						ClearStructReferencesInBP(FoundBlueprint, BlueprintsToRecompile);
						StructProperty->Struct = DuplicatedStruct;
						StructsToRegenerateReferencesFor.Add(OwnerClass);
						UStruct* OwnerStruct = StructProperty->GetOwnerStruct();
						if (OwnerStruct != OwnerClass)
						{
							StructsToRegenerateReferencesFor.Add(OwnerStruct);
						}
					}
				}
				else if (UUserDefinedStruct* OwnerUDS = Cast<UUserDefinedStruct>(StructProperty->GetOwnerStruct()))
				{
					check(OwnerUDS != DuplicatedStruct);
					const bool bValidStruct = (OwnerUDS->GetOutermost() != GetTransientPackage())
						&& IsValid(OwnerUDS)
						&& (EUserDefinedStructureStatus::UDSS_Duplicate != OwnerUDS->Status.GetValue());

					if (bValidStruct)
					{
						ChangedStructs.AddUnique(OwnerUDS);

						if (FStructureEditorUtils::FStructEditorManager::ActiveChange != FStructureEditorUtils::EStructureEditorChangeInfo::DefaultValueChanged)
						{
							// Don't change this for a default value only change, it won't get correctly replaced later
							StructProperty->Struct = DuplicatedStruct;
							StructsToRegenerateReferencesFor.Add(OwnerUDS);
						}
					}
				}
				else if (UPropertyBag* OwnerPropertyBag = Cast<UPropertyBag>(StructProperty->GetOwnerStruct()))
				{
					static bool Value = true;
					if (Value == false)
					{
						continue;
					}
					check(OwnerPropertyBag != DuplicatedStruct);

					ChangedStructs.AddUnique(OwnerPropertyBag);
					StructProperty->Struct = DuplicatedStruct;

					const FProperty* StructPropertyOwner = StructProperty->GetTypedOwner<FProperty>();
					const FProperty* PropertyBagField = StructPropertyOwner != nullptr ? StructPropertyOwner : StructProperty;
					if (ensure(PropertyBagField != nullptr))
					{
						const FPropertyBagPropertyDesc* Desc = OwnerPropertyBag->FindPropertyDescByProperty(PropertyBagField);
						if (ensure(Desc != nullptr))
						{
							if (Desc->ValueTypeObject == StructureToReinstance)
							{
								const_cast<FPropertyBagPropertyDesc*>(Desc)->ValueTypeObject = DuplicatedStruct;
							}
							if (Desc->KeyTypeObject == StructureToReinstance)
							{
								const_cast<FPropertyBagPropertyDesc*>(Desc)->KeyTypeObject = DuplicatedStruct;
							}
						}
					}
				}
				else
				{
					UE_LOGF(LogK2Compiler, Error, "ReplaceStructWithTempDuplicate unknown owner");
				}
			}
		}

		// Make sure we update the list of objects referenced by structs after we replaced the struct in FStructProperties
		for (UStruct* Struct : StructsToRegenerateReferencesFor)
		{
			Struct->CollectBytecodeAndPropertyReferencedObjects();
		}

		for (UBlueprint* Blueprint : TObjectRange<UBlueprint>(RF_ClassDefaultObject, /** bIncludeDerivedClasses */ true, /** InternalExcludeFlags */ EInternalObjectFlags::Garbage))
		{
			if (Blueprint && !BlueprintsToRecompile.Contains(Blueprint))
			{
				FBlueprintEditorUtils::EnsureCachedDependenciesUpToDate(Blueprint);
				if (Blueprint->CachedUDSDependencies.Contains(StructureToReinstance))
				{
					ClearStructReferencesInBP(Blueprint, BlueprintsToRecompile);
				}
			}
		}
	}

	static void CleanAndSanitizeStruct(UUserDefinedStruct* StructToClean)
	{
		check(StructToClean);

		if (UUserDefinedStructEditorData* EditorData = Cast<UUserDefinedStructEditorData>(StructToClean->EditorData))
		{
			EditorData->CleanDefaultInstance();
		}

		if (FStructureEditorUtils::FStructEditorManager::ActiveChange != FStructureEditorUtils::EStructureEditorChangeInfo::DefaultValueChanged)
		{
			StructToClean->SetSuperStruct(nullptr);
			StructToClean->Children = nullptr;
			StructToClean->DestroyChildPropertiesAndResetPropertyLinks();
			StructToClean->Script.Empty();
			StructToClean->MinAlignment = 0;
			StructToClean->ScriptAndPropertyObjectReferences.Empty();
			StructToClean->ErrorMessage.Empty();
			StructToClean->SetStructTrashed(true);
			StructToClean->PropertyIDs.Empty();
		}
	}

	static void LogError(UUserDefinedStruct* Struct, FCompilerResultsLog& MessageLog, const FString& ErrorMsg)
	{
		MessageLog.Error(*ErrorMsg);
		if (Struct && Struct->ErrorMessage.IsEmpty())
		{
			Struct->ErrorMessage = ErrorMsg;
		}
	}

	static void CreateVariables(TNotNull<UUserDefinedStruct*> Struct, TNotNull<const UEdGraphSchema_K2*> Schema, FCompilerResultsLog& MessageLog)
	{
		//FKismetCompilerUtilities::LinkAddedProperty push property to begin, so we revert the order
		for (int32 VarDescIdx = FStructureEditorUtils::GetVarDesc(Struct).Num() - 1; VarDescIdx >= 0; --VarDescIdx)
		{
			FStructVariableDescription& VarDesc = FStructureEditorUtils::GetVarDesc(Struct)[VarDescIdx];
			VarDesc.bInvalidMember = true;

			FEdGraphPinType VarType = VarDesc.ToPinType();

			FString ErrorMsg;
			if(!FStructureEditorUtils::CanHaveAMemberVariableOfType(Struct, VarType, &ErrorMsg))
			{
				LogError(
					Struct,
					MessageLog,
					FText::Format(
						LOCTEXT("StructureGeneric_ErrorFmt", "Structure: {0} Error: {1}"),
						FText::FromString(Struct->GetFullName()),
						FText::FromString(ErrorMsg)
					).ToString()
				);
				continue;
			}

			FProperty* VarProperty = nullptr;

			bool bIsNewVariable = false;
			if (FStructureEditorUtils::FStructEditorManager::ActiveChange == FStructureEditorUtils::EStructureEditorChangeInfo::DefaultValueChanged)
			{
				VarProperty = FindFProperty<FProperty>(Struct, VarDesc.VarName);
				if (!ensureMsgf(VarProperty, TEXT("Could not find the expected property (%s); was the struct (%s) unexpectedly sanitized?"), *VarDesc.VarName.ToString(), *Struct->GetName()))
				{
					VarProperty = FKismetCompilerUtilities::CreatePropertyOnScope(Struct, VarDesc.VarName, VarType, NULL, CPF_None, Schema, MessageLog);
					bIsNewVariable = true;
				}
			}
			else
			{
				VarProperty = FKismetCompilerUtilities::CreatePropertyOnScope(Struct, VarDesc.VarName, VarType, NULL, CPF_None, Schema, MessageLog);
				bIsNewVariable = true;
			}

			if (VarProperty == nullptr)
			{
				LogError(
					Struct,
					MessageLog,
					FText::Format(
						LOCTEXT("VariableInvalidType_ErrorFmt", "The variable {0} declared in {1} has an invalid type {2}"),
						FText::FromName(VarDesc.VarName),
						FText::FromString(Struct->GetName()),
						UEdGraphSchema_K2::TypeToText(VarType)
					).ToString()
				);
				continue;
			}
			else if (bIsNewVariable)
			{
				FKismetCompilerUtilities::LinkAddedProperty(Struct, VarProperty);
			}

			if (VarDesc.VarGuid.IsValid())
			{
				const UE::StructUtils::FUserDefinedPropertyID* Found = Struct->PropertyIDs.FindByPredicate(
					[ID = VarDesc.VarGuid](const UE::StructUtils::FUserDefinedPropertyID& Other)
					{
						return Other.ID == ID;
					});
				if (Found != nullptr && bIsNewVariable)
				{
					LogError(
						Struct,
						MessageLog,
						FText::Format(
							LOCTEXT("VariableDuplicateID_ErrorFmt", "The variable {0} declared in {1} has the same ID as variable {2}."),
							FText::FromName(VarDesc.VarName),
							FText::FromString(Struct->GetName()),
							FText::FromName(Found->Name)
						).ToString()
					);
				}
				else if (Found == nullptr)
				{
					Struct->PropertyIDs.Add(UE::StructUtils::FUserDefinedPropertyID{ .ID = VarDesc.VarGuid , .Name = VarProperty->GetFName() });
				}
			}
			
			VarProperty->SetPropertyFlags(CPF_Edit | CPF_BlueprintVisible);
			if (VarDesc.bDontEditOnInstance)
			{
				VarProperty->SetPropertyFlags(CPF_DisableEditOnInstance);
			}
			if (VarDesc.bEnableSaveGame)
			{
				VarProperty->SetPropertyFlags(CPF_SaveGame);
			}
			if (VarDesc.bEnableMultiLineText)
			{
				VarProperty->SetMetaData("MultiLine", TEXT("true"));
			}
			if (VarDesc.bEnable3dWidget)
			{
				VarProperty->SetMetaData(FEdMode::MD_MakeEditWidget, TEXT("true"));
			}
			VarProperty->SetMetaData(TEXT("DisplayName"), *VarDesc.FriendlyName);
			VarProperty->SetMetaData(FBlueprintMetadata::MD_Tooltip, *VarDesc.ToolTip);
			VarProperty->AppendMetaData(VarDesc.MetaData);
			VarProperty->RepNotifyFunc = NAME_None;

			if (!VarDesc.DefaultValue.IsEmpty())
			{
				VarProperty->SetMetaData(TEXT("MakeStructureDefaultValue"), *VarDesc.DefaultValue);
			}
			VarDesc.CurrentDefaultValue = VarDesc.DefaultValue;

			VarDesc.bInvalidMember = false;

			if (VarProperty->HasAnyPropertyFlags(CPF_InstancedReference | CPF_ContainsInstancedReference))
			{
				Struct->StructFlags = EStructFlags(Struct->StructFlags | STRUCT_HasInstancedReference);
			}

			if (VarType.PinSubCategoryObject.IsValid())
			{
				const UClass* ClassObject = Cast<UClass>(VarType.PinSubCategoryObject.Get());

				if (ClassObject && ClassObject->IsChildOf(AActor::StaticClass()) && (VarType.PinCategory == UEdGraphSchema_K2::PC_Object || VarType.PinCategory == UEdGraphSchema_K2::PC_Interface))
				{
					// NOTE: Right now the code that stops hard AActor references from being set in unsafe places is tied to this flag,
					// which is not generally respected in other places for struct properties
					VarProperty->PropertyFlags |= CPF_DisableEditOnTemplate;
				}
				else
				{
					// clear the disable-default-value flag that might have been present (if this was an AActor variable before)
					VarProperty->PropertyFlags &= ~(CPF_DisableEditOnTemplate);
				}
			}
		}
	}

	static void InnerCompileStruct(UUserDefinedStruct* Struct, const class UEdGraphSchema_K2* K2Schema, class FCompilerResultsLog& MessageLog)
	{
		check(Struct);
		const int32 ErrorNum = MessageLog.NumErrors;

		UUserDefinedStructEditorData* EditorData = CastChecked<UUserDefinedStructEditorData>(Struct->EditorData);

		UPackage* Package = Struct->GetOutermost();
		if (ensure(Package))
		{
			TMap<FName, FString>& Map = Package->GetMetaData().ObjectMetaDataMap.FindOrAdd(FSoftObjectPath(Struct));
			Map.Empty();
			Map.Append(EditorData->MetaData);
			Map.Add(FBlueprintMetadata::MD_Tooltip, *FStructureEditorUtils::GetTooltip(Struct));
		}

		CreateVariables(Struct, K2Schema, MessageLog);

		Struct->Bind();
		Struct->StaticLink(true);

		if (Struct->GetStructureSize() <= 0)
		{
			LogError(
				Struct,
				MessageLog,
				FText::Format(
					LOCTEXT("StructurEmpty_ErrorFmt", "Structure '{0}' is empty "),
					FText::FromString(Struct->GetFullName())
				).ToString()
			);
		}

		FString DefaultInstanceError;
		EditorData->RecreateDefaultInstance(&DefaultInstanceError);
		if (!DefaultInstanceError.IsEmpty())
		{
			LogError(Struct, MessageLog, DefaultInstanceError);
		}

		const bool bNoErrorsDuringCompilation = (ErrorNum == MessageLog.NumErrors);
		Struct->Status = bNoErrorsDuringCompilation ? EUserDefinedStructureStatus::UDSS_UpToDate : EUserDefinedStructureStatus::UDSS_Error;
	}

	static bool ShouldBeCompiled(const UUserDefinedStruct* Struct)
	{
		if (Struct && (EUserDefinedStructureStatus::UDSS_UpToDate == Struct->Status))
		{
			return false;
		}
		return true;
	}

	static void BuildDependencyMapAndCompile(const TArray<UScriptStruct*>& ChangedStructs, FCompilerResultsLog& MessageLog)
	{
		struct FDependencyMapEntry
		{
			UScriptStruct* Struct;
			TSet<const UScriptStruct*> StructuresToWaitFor;

			FDependencyMapEntry() : Struct(NULL) {}

			void Initialize(UScriptStruct* ChangedStruct, const TArray<UScriptStruct*>& AllChangedStructs)
			{ 
				Struct = ChangedStruct;
				check(Struct);

				if (UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(Struct))
				{
					for (FStructVariableDescription& VarDesc : FStructureEditorUtils::GetVarDesc(UserDefinedStruct))
					{
						UScriptStruct* StructType = Cast<UScriptStruct>(VarDesc.SubCategoryObject.Get());
						if (StructType && (VarDesc.Category == UEdGraphSchema_K2::PC_Struct) && AllChangedStructs.Contains(StructType))
						{
							StructuresToWaitFor.Add(StructType);
						}
					}
				}
				else if (UPropertyBag* PropertyBag = Cast<UPropertyBag>(Struct))
				{
					for (const FPropertyBagPropertyDesc& Desc : PropertyBag->GetPropertyDescs())
					{
						if (Desc.ValueType == EPropertyBagPropertyType::Struct && AllChangedStructs.Contains(Desc.ValueTypeObject))
						{
							StructuresToWaitFor.Add(CastChecked<const UScriptStruct>(Desc.ValueTypeObject));
						}
						if (Desc.KeyType == EPropertyBagPropertyType::Struct && AllChangedStructs.Contains(Desc.KeyTypeObject))
						{
							StructuresToWaitFor.Add(CastChecked<const UScriptStruct>(Desc.KeyTypeObject));
						}
					}
				}
				else
				{
					ensure(false);
				}
			}
		};

		TArray<FDependencyMapEntry> DependencyMap;
		for (UScriptStruct* ChangedStruct : ChangedStructs)
		{
			DependencyMap.Add(FDependencyMapEntry());
			DependencyMap.Last().Initialize(ChangedStruct, ChangedStructs);
		}

		UE::StructUtils::FStructReinstancer* StructReinstancer = UE::StructUtils::FStructReinstancer::GetInstance();
		check(StructReinstancer);

		while (DependencyMap.Num())
		{
			int32 StructureToCompileIndex = INDEX_NONE;
			for (int32 EntryIndex = 0; EntryIndex < DependencyMap.Num(); ++EntryIndex)
			{
				if(0 == DependencyMap[EntryIndex].StructuresToWaitFor.Num())
				{
					StructureToCompileIndex = EntryIndex;
					break;
				}
			}
			check(INDEX_NONE != StructureToCompileIndex);
			UScriptStruct* Struct = DependencyMap[StructureToCompileIndex].Struct;
			check(Struct);

			if (UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(Struct))
			{
				FUserDefinedStructureCompilerInner::CleanAndSanitizeStruct(UserDefinedStruct);
				FUserDefinedStructureCompilerInner::InnerCompileStruct(UserDefinedStruct, GetDefault<UEdGraphSchema_K2>(), MessageLog);
				if (UUserDefinedStructEditorData* EditorData = Cast<UUserDefinedStructEditorData>(UserDefinedStruct->EditorData))
				{
					// Ensure that editor data is in sync w/ the current default instance (if valid) so that it can be reinitialized later.
					EditorData->RefreshValuesFromDefaultInstance();
				}

				StructReinstancer->SetCompiledStruct(Struct, Struct);
			}
			else if (UPropertyBag* PropertyBag = Cast<UPropertyBag>(Struct))
			{
				//Property bag properties desc use the duplicated UDS or original property bag
				//replace struct with compiled struct
				TArrayView<const FPropertyBagPropertyDesc> PreviousPropertyDescs = PropertyBag->GetPropertyDescs();
				TArray<FPropertyBagPropertyDesc> PropertyDescs = TArray<FPropertyBagPropertyDesc>(PreviousPropertyDescs.GetData(), PreviousPropertyDescs.Num());
				for (FPropertyBagPropertyDesc& PropDesc : PropertyDescs)
				{
					if (PropDesc.ValueType == EPropertyBagPropertyType::Struct)
					{
						if (const UScriptStruct* ValueStruct = Cast<const UScriptStruct>(PropDesc.ValueTypeObject))
						{
							if (const UScriptStruct* CompiledStruct = StructReinstancer->GetCompiledReinstantingStruct(ValueStruct))
							{
								PropDesc.ValueTypeObject = CompiledStruct;
							}
						}
					}

					if (PropDesc.KeyType == EPropertyBagPropertyType::Struct)
					{
						if (const UScriptStruct* ValueStruct = Cast<const UScriptStruct>(PropDesc.KeyTypeObject))
						{
							if (const UScriptStruct* CompiledStruct = StructReinstancer->GetCompiledReinstantingStruct(ValueStruct))
							{
								PropDesc.KeyTypeObject = CompiledStruct;
							}
						}
					}
				}

				const UPropertyBag* NewPropertyBag = const_cast<UPropertyBag*>(UPropertyBag::GetOrCreateFromDescs(PropertyDescs));
				StructReinstancer->SetCompiledStruct(Struct, NewPropertyBag);
			}

			DependencyMap.RemoveAtSwap(StructureToCompileIndex);

			for (FDependencyMapEntry& MapEntry : DependencyMap)
			{
				MapEntry.StructuresToWaitFor.Remove(Struct);
			}
		}
	}
};

void FUserDefinedStructureCompilerUtils::CompileStruct(UUserDefinedStruct* Struct, FCompilerResultsLog& MessageLog, bool bForceRecompile)
{
	if (FStructureEditorUtils::UserDefinedStructEnabled() && Struct)
	{
		struct FDuplicatedStruct
		{
			TObjectPtr<UScriptStruct> Struct;
		};
		TArray<UScriptStruct*> ChangedStructs;
		TArray<FDuplicatedStruct> DuplicatedStructsToClean;
		if (FUserDefinedStructureCompilerInner::ShouldBeCompiled(Struct) || bForceRecompile)
		{
			ChangedStructs.Add(Struct);
		}

		UE::StructUtils::FStructReinstancerScope StructReinstancer;

		TMap<UBlueprint*, FUserDefinedStructureCompilerInner::FBlueprintUserStructData> BlueprintsToRecompile;
		for (int32 StructIdx = 0; StructIdx < ChangedStructs.Num(); ++StructIdx)
		{
			UScriptStruct* ChangedStruct = ChangedStructs[StructIdx];
			if (ChangedStruct)
			{
				UUserDefinedStruct* ChangedUDS = Cast<UUserDefinedStruct>(ChangedStruct);

				UScriptStruct* DuplicatedStruct = nullptr;
				if (ChangedUDS != nullptr)
				{
					// Recompiled UDS keep the same instance. Create a duplicate struct to prevent memory corruption.
					DuplicatedStruct = FUserDefinedStructureCompilerInner::CreateDuplicate(ChangedUDS);
					DuplicatedStruct->AddToRoot();
					DuplicatedStructsToClean.Emplace(DuplicatedStruct);
					FStructureEditorUtils::BroadcastPreChange(ChangedUDS);
				}
				else
				{
					// Recompiled PropertyBag have a new instance. They can keep their same instance without causing memory coruption.
					DuplicatedStruct = ChangedStruct;
				}

				UE::StructUtils::FStructReinstancer::GetInstance()->AddStruct(ChangedStruct, DuplicatedStruct);

				FUserDefinedStructureCompilerInner::ReplaceStructWithTempDuplicate(ChangedStruct, DuplicatedStruct, BlueprintsToRecompile, ChangedStructs);
				
				if (ChangedUDS != nullptr)
				{
					ChangedUDS->Status = EUserDefinedStructureStatus::UDSS_Dirty;
				}
			}
		}

		// COLLECT ALL UOBJECT THAT REFERENCE THE STRUCTURES
		UE::StructUtils::FStructReinstancer::GetInstance()->CollectObjects();

		// COMPILE IN PROPER ORDER
		FUserDefinedStructureCompilerInner::BuildDependencyMapAndCompile(ChangedStructs, MessageLog);

		// UPDATE ALL THINGS DEPENDENT ON COMPILED STRUCTURES
		TSet<UScriptStruct*> ChangedStructsSet;
		ChangedStructsSet.Reserve(ChangedStructs.Num());
		Algo::Copy(ChangedStructs, ChangedStructsSet);
		TSet<UBlueprint*> BlueprintsThatHaveBeenRecompiled;
		FBlueprintEditorUtils::FindScriptStructsInNodes(ChangedStructsSet, [&BlueprintsThatHaveBeenRecompiled, &BlueprintsToRecompile](UBlueprint* Blueprint, UK2Node* Node)
			{
				if (Blueprint)
				{
					// The blueprint skeleton needs to be updated before we reconstruct the node
					// or else we may have member references that point to the old skeleton
					if (!BlueprintsThatHaveBeenRecompiled.Contains(Blueprint))
					{
						BlueprintsThatHaveBeenRecompiled.Add(Blueprint);
						BlueprintsToRecompile.Remove(Blueprint);

						// Reapply CDO data

						FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
					}
					Node->ReconstructNode();
				}
			}
		);

		for (TPair<UBlueprint*, FUserDefinedStructureCompilerInner::FBlueprintUserStructData>& Pair : BlueprintsToRecompile)
		{
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Pair.Key);
		}

		// RESTORE THE UOBJECT THAT REFERENCE THE STRUCTURES
		UE::StructUtils::FStructReinstancer::GetInstance()->ReinstanceObjects();

		for (UScriptStruct* ChangedStruct : ChangedStructs)
		{
			if (UUserDefinedStruct* ChangedUDS = Cast<UUserDefinedStruct>(ChangedStruct))
			{
				FStructureEditorUtils::BroadcastPostChange(ChangedUDS);
				ChangedUDS->MarkPackageDirty();
			}
		}

		for (const FDuplicatedStruct& DuplicatedStruct : DuplicatedStructsToClean)
		{
			DuplicatedStruct.Struct->RemoveFromRoot();
			DuplicatedStruct.Struct->StructFlags = EStructFlags(DuplicatedStruct.Struct->StructFlags | STRUCT_NewerVersionExists);
		}
	}
}

void FUserDefinedStructureCompilerUtils::ReplaceStructWithTempDuplicateByPredicate(
	UUserDefinedStruct* StructureToReinstance,
	TFunctionRef<bool(FStructProperty* InStructProperty)> ShouldReplaceStructInStructProperty,
	TFunctionRef<void(UStruct* InStruct)> PostReplace)
{
	if (StructureToReinstance)
	{
		UUserDefinedStruct* DuplicatedStruct = FUserDefinedStructureCompilerInner::CreateDuplicate(StructureToReinstance);
		DuplicatedStruct->AddToRoot();

		// List of unique classes and structs to regenerate
		TSet<UStruct*> StructsToRegenerateReferencesFor;

		for (TAllFieldsIterator<FStructProperty> FieldIt(RF_NoFlags, EInternalObjectFlags::Garbage); FieldIt; ++FieldIt)
		{
			FStructProperty* StructProperty = *FieldIt;
			if (StructProperty && (StructureToReinstance == StructProperty->Struct))
			{
				if(ShouldReplaceStructInStructProperty(StructProperty))
				{
					StructProperty->Struct = DuplicatedStruct;
					StructsToRegenerateReferencesFor.Add(StructProperty->GetOwnerClass());
				}
			}
		}

		for (UStruct* Struct : StructsToRegenerateReferencesFor)
		{
			Struct->CollectBytecodeAndPropertyReferencedObjects();

			PostReplace(Struct);
		}

		// as property owners are re-created, the duplicated struct will be GCed
		DuplicatedStruct->RemoveFromRoot();
	}
}

void FUserDefinedStructureCompilerUtils::ReplaceStructWithTempDuplicateSafe(UUserDefinedStruct* Struct)
{
	if (!Struct)
	{
		return;
	}
	
	UUserDefinedStruct* DuplicatedStruct = FUserDefinedStructureCompilerInner::CreateDuplicate(Struct);
	DuplicatedStruct->AddToRoot();
	ON_SCOPE_EXIT
	{
		DuplicatedStruct->RemoveFromRoot();
	};

	TSet<UStruct*> StructsToRegenerateReferencesFor;
	for (TAllFieldsIterator<FStructProperty> FieldIt(RF_NoFlags); FieldIt; ++FieldIt)
	{
		FStructProperty* StructProperty = *FieldIt;
		if (StructProperty && Struct == StructProperty->Struct)
		{
			StructProperty->Struct = DuplicatedStruct;
			StructsToRegenerateReferencesFor.Add(StructProperty->GetOwnerStruct());
		}
	}

	// all references to Struct must now be replaced with DuplicatedStruct - a safe clone - this should include Instanced Structs and property bags:
	TArray<UObject*> ReferencedObjects;
	ReferencedObjects.Add(Struct);
	TArray<UObject *> Targets;
	TSet<UObject*> IgnoredObjects;
	IgnoredObjects.Add(Struct);
	Targets = FReferencerFinder::GetAllReferencers(ReferencedObjects, &IgnoredObjects);
	TMap<UObject*, UObject*> OldToNew;
	OldToNew.Add(Struct, DuplicatedStruct);
	for(UObject* Target : Targets)
	{
		FArchiveReplaceObjectAndStructPropertyRef<UObject> ReplaceAr(Target, OldToNew);
	}

	for (UStruct* DirtyStruct : StructsToRegenerateReferencesFor)
	{
		DirtyStruct->CollectBytecodeAndPropertyReferencedObjects();
	}
}

#undef LOCTEXT_NAMESPACE
