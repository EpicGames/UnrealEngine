// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphPin.h"

struct FBlueprintCompiledStatement;

//////////////////////////////////////////////////////////////////////////

struct FBlueprintCompiledStatement;

/** A terminal in the graph (literal or variable reference) */
struct FBPTerminal
{
	FString Name;
	FEdGraphPinType Type;
	bool bIsLiteral; 
	bool bIsConst;
	bool bIsSavePersistent;
	bool bPassedByReference;

	// Source node
	UObject* Source;

	// Source pin
	UEdGraphPin* SourcePin;

	// Context->
	FBPTerminal* Context;

	// For non-literal terms, this is the FProperty being referenced (in the stack if bIsLocal set, or on the context otherwise)
	FProperty* AssociatedVarProperty;

	/** Pointer to an object literal */
	TObjectPtr<UObject> ObjectLiteral;

	/** The FText literal */
	FText TextLiteral;

	/** String representation of the default value of the property associated with this term (or path to object) */
	FString PropertyDefault;

	/** Used for MathExpression optimization. The parameter will be filled directly by a result of a function called inline. No local variable is necessary to pass the value. */
	FBlueprintCompiledStatement* InlineGeneratedParameter;

	FBPTerminal()
		: bIsLiteral(false)
		, bIsConst(false)
		, bIsSavePersistent(false)
		, bPassedByReference(false)
		, Source(nullptr)
		, SourcePin(nullptr)
		, Context(nullptr)
		, AssociatedVarProperty(nullptr)
		, ObjectLiteral(nullptr)
		, InlineGeneratedParameter(nullptr)
		, VarType(EVarType_Instanced)
		, ContextType(EContextType_Object)
	{
	}

	KISMETCOMPILER_API void CopyFromPin(UEdGraphPin* Net, FString NewName);
	void CopyFromPin(UEdGraphPin* Net, const FName NewName) { CopyFromPin(Net, NewName.ToString()); }

	bool IsTermWritable() const
	{
		return !bIsLiteral && !bIsConst;
	}

	bool IsLocalVarTerm() const
	{
		return !bIsLiteral && VarType == EVarType_Local;
	}

	void SetVarTypeLocal(bool bIsLocal = true)
	{
		VarType = bIsLocal ? EVarType_Local : EVarType_Instanced;
	}

	bool IsSparseClassDataVarTerm() const
	{
		return !bIsLiteral && VarType == EVarType_SparseClassData;
	}

	void SetVarTypeSparseClassData()
	{
		VarType = EVarType_SparseClassData;
	}

	bool IsDefaultVarTerm() const
	{
		return !bIsLiteral && VarType == EVarType_Default;
	}

	void SetVarTypeDefault(bool bIsDefault = true)
	{
		VarType = bIsDefault ? EVarType_Default : EVarType_Instanced;
	}

	bool IsInstancedVarTerm() const
	{
		return !bIsLiteral && VarType == EVarType_Instanced;
	}

	bool IsClassContextType() const
	{
		return ContextType == EContextType_Class;
	}

	void SetContextTypeClass(bool bIsClassContext = true)
	{
		ContextType = bIsClassContext ? EContextType_Class : EContextType_Object;
	}

	bool IsStructContextType() const
	{
		return ContextType == EContextType_Struct;
	}

	void SetContextTypeStruct(bool bIsStructContext = true)
	{
		ContextType = bIsStructContext ? EContextType_Struct : EContextType_Object;
	}

	bool IsObjectContextType() const
	{
		return ContextType == EContextType_Object;
	}

private:
	// Variable reference types (mutually-exclusive)
	enum EVarType
	{
		EVarType_Local,
		EVarType_Default,
		EVarType_Instanced,
		EVarType_SparseClassData,
	};

	// For non-literal terms, this is set to the type of variable reference
	EVarType VarType;

	// Context types (mutually-exclusive)
	enum EContextType
	{
		EContextType_Class,
		EContextType_Struct,
		EContextType_Object,
	};

	// If this term is also a context, this indicates which type of context it is
	EContextType ContextType;
};

//////////////////////////////////////////////////////////////////////////
