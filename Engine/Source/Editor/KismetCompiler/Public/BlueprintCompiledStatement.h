// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

//////////////////////////////////////////////////////////////////////////
// FBlueprintCompiledStatement

enum EKismetCompiledStatementType
{
	KCST_Nop = 0,
	// [wiring =] TargetObject->FunctionToCall(wiring)
	KCST_CallFunction = 1,
	// TargetObject->TargetProperty = [wiring]
	KCST_Assignment = 2,
	// One of the other types with a compilation error during statement generation
	KCST_CompileError = 3,
	// goto TargetLabel
	KCST_UnconditionalGoto = 4,
	// FlowStack.Push(TargetLabel)
	KCST_PushState = 5,
	// [if (!TargetObject->TargetProperty)] goto TargetLabel
	KCST_GotoIfNot = 6,
	// return TargetObject->TargetProperty
	KCST_Return = 7,
	// if (FlowStack.Num()) { NextState = FlowStack.Pop; } else { return; }
	KCST_EndOfThread = 8,
	// Comment
	KCST_Comment = 9,
	// NextState = LHS;
	KCST_ComputedGoto = 10,
	// [if (!TargetObject->TargetProperty)] { same as KCST_EndOfThread; }
	KCST_EndOfThreadIfNot = 11,
	// NOP with recorded address
	KCST_DebugSite = 12,
	// TargetInterface(TargetObject)
	KCST_CastObjToInterface = 13,
	// Cast<TargetClass>(TargetObject)
	KCST_DynamicCast = 14,
	// (TargetObject != None)
	KCST_ObjectToBool = 15,
	// TargetDelegate->Add(EventDelegate)
	KCST_AddMulticastDelegate = 16,
	// TargetDelegate->Clear()
	KCST_ClearMulticastDelegate = 17,
	// NOP with recorded address (never a step target)
	KCST_WireTraceSite = 18,
	// Creates simple delegate
	KCST_BindDelegate = 19,
	// TargetDelegate->Remove(EventDelegate)
	KCST_RemoveMulticastDelegate = 20,
	// TargetDelegate->Broadcast(...)
	KCST_CallDelegate = 21,
	// Creates and sets an array literal term
	KCST_CreateArray = 22,
	// TargetInterface(Interface)
	KCST_CrossInterfaceCast = 23,
	// Cast<TargetClass>(TargetObject)
	KCST_MetaCast = 24,
	KCST_AssignmentOnPersistentFrame = 25,
	// Cast<TargetClass>(TargetInterface)
	KCST_CastInterfaceToObj = 26,
	// goto ReturnLabel
	KCST_GotoReturn = 27,
	// [if (!TargetObject->TargetProperty)] goto TargetLabel
	KCST_GotoReturnIfNot = 28,
	KCST_SwitchValue = 29,
	
	KCST_DoubleToFloatCast = 30,
	KCST_FloatToDoubleCast = 31,

	//~ Kismet instrumentation extensions:

	// Instrumented event
	KCST_InstrumentedEvent,
	// Instrumented event stop
	KCST_InstrumentedEventStop,
	// Instrumented pure node entry
	KCST_InstrumentedPureNodeEntry,
	// Instrumented wiretrace entry
	KCST_InstrumentedWireEntry,
	// Instrumented wiretrace exit
	KCST_InstrumentedWireExit,
	// Instrumented state push
	KCST_InstrumentedStatePush,
	// Instrumented state restore
	KCST_InstrumentedStateRestore,
	// Instrumented state reset
	KCST_InstrumentedStateReset,
	// Instrumented state suspend
	KCST_InstrumentedStateSuspend,
	// Instrumented state pop
	KCST_InstrumentedStatePop,
	// Instrumented tunnel exit
	KCST_InstrumentedTunnelEndOfThread,

	KCST_ArrayGetByRef,
	KCST_CreateSet,
	KCST_CreateMap,
};

//@TODO: Too rigid / icky design
struct FBlueprintCompiledStatement
{
	FBlueprintCompiledStatement()
		: Type(KCST_Nop)
		, FunctionContext(NULL)
		, FunctionToCall(NULL)
		, TargetLabel(NULL)
		, UbergraphCallIndex(-1)
		, LHS(NULL)
		, bIsJumpTarget(false)
		, bIsInterfaceContext(false)
		, bIsParentContext(false)
		, ExecContext(NULL)
	{
	}

	EKismetCompiledStatementType Type;

	// Object that the function should be called on, or NULL to indicate self (KCST_CallFunction)
	struct FBPTerminal* FunctionContext;

	// Function that executes the statement (KCST_CallFunction)
	UFunction* FunctionToCall;

	// Target label (KCST_Goto, or KCST_CallFunction that requires an ubergraph reference)
	FBlueprintCompiledStatement* TargetLabel;

	// The index of the argument to replace (only used when KCST_CallFunction has a non-NULL TargetLabel)
	int32 UbergraphCallIndex;

	// Destination of assignment statement or result from function call
	struct FBPTerminal* LHS;

	// Argument list of function call or source of assignment statement
	TArray<struct FBPTerminal*> RHS;

	// Is this node a jump target?
	bool bIsJumpTarget;

	// Is this node an interface context? (KCST_CallFunction)
	bool bIsInterfaceContext;

	// Is this function called on a parent class (super, etc)?  (KCST_CallFunction)
	bool bIsParentContext;

	// Exec pin about to execute (KCST_WireTraceSite)
	class UEdGraphPin* ExecContext;

	// Pure node output pin(s) linked to exec node input pins (KCST_InstrumentedPureNodeEntry)
	TArray<class UEdGraphPin*> PureOutputContextArray;

	// Comment text
	FString Comment;
};
