// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"

#define UE_API KISMETCOMPILER_API

class FCompilerResultsLog;
class UEdGraph;

//////////////////////////////////////////////////////////////////////////

class FGraphCompilerContext
{
public:
	// Compiler message log (errors, warnings, notes)
	FCompilerResultsLog& MessageLog;

protected:

	// Schema implementation

	/** Validates that the interconnection between two pins is schema compatible */
	UE_API virtual void ValidateLink(const UEdGraphPin* PinA, const UEdGraphPin* PinB) const;

	/** Validate that the wiring for a single pin is schema compatible */
	UE_API virtual void ValidatePin(const UEdGraphPin* Pin) const;

	/** Validates that the node is schema compatible */
	UE_API virtual void ValidateNode(const UEdGraphNode* Node) const;

	/** Can this node be ignored for further processing? */
	virtual bool CanIgnoreNode(const UEdGraphNode* Node) const
	{
		return false;
	}

	/** Should this node be kept even if it's not reached? */
	virtual bool ShouldForceKeepNode(const UEdGraphNode* Node) const
	{
		return false;
	}

	/** Does this pin potentially participate in data dependencies? */
	virtual bool PinIsImportantForDependancies(const UEdGraphPin* Pin) const
	{
		return false;
	}

	FGraphCompilerContext(FCompilerResultsLog& InMessageLog)
		: MessageLog(InMessageLog)
	{
	}

	/** Performs standard validation on the graph (outputs point to inputs, no more than one connection to each input, types match on both ends, etc...) */
	UE_API bool ValidateGraphIsWellFormed(UEdGraph* Graph) const;

	/**
	 * Scans a graph for a node of the specified class.  Can optionally continue scanning and print errors if additional nodes of the same category are found.
	 */
	UE_API UEdGraphNode* FindNodeByClass(const UEdGraph* Graph, TSubclassOf<UEdGraphNode>  NodeClass, bool bExpectedUnique) const;

	/** Prunes any nodes that weren't visited from the graph, printing out a warning */
	UE_API virtual void PruneIsolatedNodes(const TArray<UEdGraphNode*>& RootSet, TArray<UEdGraphNode*>& GraphNodes);

	/**
	 * Performs a topological sort on the graph of nodes passed in (which is expected to form a DAG), scheduling them.
	 * If there are cycles or unconnected nodes present in the graph, an error will be output for each node that failed to be scheduled.
	 */
	UE_API void CreateExecutionSchedule(const TArray<UEdGraphNode*>& GraphNodes, /*out*/ TArray<UEdGraphNode*>& LinearExecutionSchedule) const;

	/** Counts the number of incoming edges this node has (along all input pins) */
	int32 CountIncomingEdges(const UEdGraphNode* Node) const
	{
		int32 NumEdges = 0;
		for (int32 PinIndex = 0; PinIndex < Node->Pins.Num(); ++PinIndex)
		{
			const UEdGraphPin* Pin = Node->Pins[PinIndex];
			if ((Pin->Direction == EGPD_Input) && PinIsImportantForDependancies(Pin))
			{
				NumEdges += Pin->LinkedTo.Num();
			}
		}

		return NumEdges;
	}
};

//////////////////////////////////////////////////////////////////////////

#undef UE_API
