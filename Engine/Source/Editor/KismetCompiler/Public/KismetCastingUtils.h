// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintCompiledStatement.h"
#include "CoreMinimal.h"

class UEdGraphNode;
class UEdGraphPin;
class UFunction;
struct FBPTerminal;
struct FKismetFunctionContext;

namespace UE::KismetCompiler::CastingUtils
{
	enum class FloatingPointCastType
	{
		None,
		FloatToDouble,
		DoubleToFloat,
		Container,
		Struct
	};

	struct FConversion
	{
		FloatingPointCastType Type = FloatingPointCastType::None;
		UFunction* Function = nullptr;
	};

	struct FImplicitCastParams
	{
		FConversion Conversion;
		FBPTerminal* TargetTerminal = nullptr;
		UEdGraphNode* TargetNode = nullptr;
	};

	/**
	 * Directly creates a new FBPTerminal that holds the result of an implicit cast.
	 * Internally, this is used by InsertImplicitCastStatement, but it can be used by any node
	 * that has special semantics for handling implicit casts. The terminal is guaranteed to have 
	 * a unique name in the context.
	 * 
	 * @param Context - Current function context to add the terminal to.
	 * @param Net - The corresponding pin that we'll copy type details from.
	 * @param SourceNode - The source node for the terminal. This is usually the Net's owner, but it can be different if necessary.
	 * @return A new FBPTerminal for the temporary variable that holds the casted result.
	 */
	KISMETCOMPILER_API FBPTerminal* MakeImplicitCastTerminal(FKismetFunctionContext& Context, UEdGraphPin* Net, UEdGraphNode* SourceNode = nullptr);

	/**
	 * Analyzes the NetMap of the current function context for potential implicit casts.
	 * If any are found, they're added to ImplicitCastMap in the context.
	 * After function compilation, the Kismet compiler will validate that the map is empty.
	 * It's up to the nodes to check the map and insert cast statements where necessary.
	 * 
	 * @param Context - Current function context to analyze. Assumes the NetMap has been populated.
	 */
	KISMETCOMPILER_API void RegisterImplicitCasts(FKismetFunctionContext& Context);

	/**
	 * Utility function used by nodes for inserting implicit cast statements.
	 * A compiled cast statement is created on the target node with the given parameters.
	 * 
	 * @param Context - Current function context to analyze. Assumes the ImplicitCastMap has been populated.
	 * @param CastParams - Parameters for the cast operation.
	 * @param RHSTerm - The current terminal that should have its data read from.
	 */
	KISMETCOMPILER_API void InsertImplicitCastStatement(FKismetFunctionContext& Context,
														const FImplicitCastParams& CastParams,
														FBPTerminal* RHSTerm);

	/**
	 * Utility function used by nodes for inserting implicit cast statements.
	 * Similar to above, but the cast parameters are read from the context's cast map using
	 * a destination pin as the key. If no entry exists in the map, null is returned.
	 *
	 * @param Context - Current function context to analyze. Assumes the ImplicitCastMap has been populated.
	 * @param DestinationPin - Used as a key in the ImplicitCastMap. These pins are always inputs.
	 * @param RHSTerm - The current terminal that should have its data read from.
	 * @return A new FBPTerminal for the temporary variable that holds the casted result (if one exists).
	 */
	KISMETCOMPILER_API FBPTerminal* InsertImplicitCastStatement(FKismetFunctionContext& Context,
																UEdGraphPin* DestinationPin,
																FBPTerminal* RHSTerm);

	/**
	 * Removes the specific UEdGraphPin from the context's implicit cast map.
	 * In most cases, InsertImplicitCastStatement should be used to remove the cast map entry.
	 * However, some nodes need to implement custom behavior for casting.
	 * 
	 * @param Context - Current function context to analyze. Assumes the ImplicitCastMap has been populated.
	 * @param DestinationPin - Used as a key in the ImplicitCastMap. These pins are always inputs.
	 * @return True if DestinationPin was found in the ImplicitCastMap.
	 */
	KISMETCOMPILER_API bool RemoveRegisteredImplicitCast(FKismetFunctionContext& Context, const UEdGraphPin* DestinationPin);

	/**
	 * Retrieves the conversion type needed between two arbitrary pins (if necessary). Specifically, this indicates if either
	 * a narrowing or widening cast is needed between a float or a double type (including containers). In addition to the
	 * corresponding FloatingPointCastType that represents the cast type, a UFunction* may be included for container and
	 * struct conversions.
	 * 
	 * @param SourcePin - The source pin to compare.
	 * @param DestinationPin - The destination pin to compare.
	 * @return A new FConversion containing the cast information.
	 */
	KISMETCOMPILER_API FConversion GetFloatingPointConversion(const UEdGraphPin& SourcePin, const UEdGraphPin& DestinationPin);

} // UE::KismetCompiler::CastingUtils

