// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/IndirectArray.h"
#include "UObject/Class.h"
#include "EdGraphSchema_K2.h"
#include "BPTerminal.h"
#include "BlueprintCompiledStatement.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCastingUtils.h"

class Error;
class UBlueprint;
class UBlueprintGeneratedClass;
class UK2Node_FunctionEntry;
struct FNetNameMapping;

namespace KismetCompilerDebugOptions
{
	//@TODO: K2: Turning this off is probably broken due to state merging not working with the current code generation
	enum { DebuggingCompiler = 1 };

	// Should the compiler emit node comments to the backends?
	enum { EmitNodeComments = DebuggingCompiler };
}

enum ETerminalSpecification
{
	TS_Unspecified,
	TS_Literal,
	TS_ForcedShared,
};

//////////////////////////////////////////////////////////////////////////
// FKismetFunctionContext

struct FKismetFunctionContext
{
public:
	/** Blueprint source */
	UBlueprint* Blueprint;

	UEdGraph* SourceGraph;

	// The nominal function entry point
	UK2Node_FunctionEntry* EntryPoint;

	UFunction* Function;
	UBlueprintGeneratedClass* NewClass;

	// Linear execution schedule
	TArray<UEdGraphNode*> LinearExecutionList;

	FCompilerResultsLog& MessageLog;
	const UEdGraphSchema_K2* Schema;

	// An UNORDERED listing of all statements (used for cleaning up the dynamically allocated statements)
	TArray< FBlueprintCompiledStatement* > AllGeneratedStatements;

	// Individual execution lists for every node that generated code to be consumed by the backend
	TMap<UEdGraphNode*, TArray<FBlueprintCompiledStatement*>> StatementsPerNode;

	// Goto fixup requests (each statement (key) wants to goto the first statement attached to the exec out-pin (value))
	TMap<FBlueprintCompiledStatement*, UEdGraphPin*> GotoFixupRequestMap;

	TIndirectArray<FBPTerminal> Parameters;
	TIndirectArray<FBPTerminal> Results;
	TIndirectArray<FBPTerminal> VariableReferences;
	TIndirectArray<FBPTerminal> PersistentFrameVariableReferences;
	TIndirectArray<FBPTerminal> Literals;
	TIndirectArray<FBPTerminal> Locals;
	TIndirectArray<FBPTerminal> EventGraphLocals;
	TIndirectArray<FBPTerminal>	LevelActorReferences;
	TIndirectArray<FBPTerminal>	InlineGeneratedValues; // A function generating the parameter will be called inline. The value won't be stored in a local variable.

	// Map from a net to an term (either a literal or a storage location)
	TMap<UEdGraphPin*, FBPTerminal*> NetMap;
	TMap<UEdGraphPin*, FBPTerminal*> LiteralHackMap;

	// Contains a map of destination pins that will need an implicit cast to either a float or double
	TMap<UEdGraphPin*, UE::KismetCompiler::CastingUtils::FImplicitCastParams> ImplicitCastMap;

	bool bIsUbergraph;
	bool bCannotBeCalledFromOtherKismet;
	bool bIsInterfaceStub;
	bool bIsConstFunction;
	bool bEnforceConstCorrectness;
	bool bCreateDebugData;
	bool bIsSimpleStubGraphWithNoParams;
	uint32 NetFlags;
	FName DelegateSignatureName;

	// If this function is an event stub, then this points to the node in the ubergraph that caused the stub to exist
	UEdGraphNode* SourceEventFromStubGraph;

	// Map from a name to the number of times it's been 'created' (same nodes create the same local variable names, so they need something appended)
	struct FNetNameMapping* NetNameMap;
	bool bAllocatedNetNameMap;

	//Does this function use requires FlowStack ?
	bool bUseFlowStack;
public:
	KISMETCOMPILER_API FKismetFunctionContext(FCompilerResultsLog& InMessageLog, const UEdGraphSchema_K2* InSchema, UBlueprintGeneratedClass* InNewClass, UBlueprint* InBlueprint);

	KISMETCOMPILER_API ~FKismetFunctionContext();

	void SetExternalNetNameMap(FNetNameMapping* NewMap);

	bool IsValid() const
	{
		return (Function != NULL) && (EntryPoint != NULL) && (SourceGraph != NULL);
	}

	bool IsEventGraph() const
	{
		return bIsUbergraph;
	}

	void MarkAsEventGraph()
	{
		bIsUbergraph = true;
	}

	void MarkAsInternalOrCppUseOnly()
	{
		bCannotBeCalledFromOtherKismet = true;
	}

	bool CanBeCalledByKismet() const
	{
		return !bCannotBeCalledFromOtherKismet;
	}

	void MarkAsInterfaceStub()
	{
		bIsInterfaceStub = true;
	}

	void MarkAsConstFunction(bool bInEnforceConstCorrectness)
	{
		bIsConstFunction = true;
		bEnforceConstCorrectness = bInEnforceConstCorrectness;
	}

	bool IsInterfaceStub() const
	{
		return bIsInterfaceStub;
	}

	void SetDelegateSignatureName(FName InName)
	{
		check((DelegateSignatureName == NAME_None) && (InName != NAME_None));
		DelegateSignatureName = InName;
	}

	bool IsDelegateSignature()
	{
		return (DelegateSignatureName != NAME_None);
	}

	bool IsConstFunction() const
	{
		return bIsConstFunction;
	}

	bool EnforceConstCorrectness() const
	{
		return bEnforceConstCorrectness;
	}

	void MarkAsNetFunction(uint32 InFunctionFlags)
	{
		NetFlags = InFunctionFlags & (FUNC_NetFuncFlags);
	}

	bool IsDebuggingOrInstrumentationRequired() const
	{
		return bCreateDebugData;
	}

	EKismetCompiledStatementType GetWireTraceType() const
	{
		return KCST_WireTraceSite;
	}

	EKismetCompiledStatementType GetBreakpointType() const
	{
		return KCST_DebugSite;
	}

	uint32 GetNetFlags() const
	{
		return NetFlags;
	}

	FBPTerminal* RegisterLiteral(UEdGraphPin* Net)
	{
		FBPTerminal* Term = new FBPTerminal();
		Literals.Add(Term);
		Term->CopyFromPin(Net, Net->DefaultValue);
		Term->ObjectLiteral = Net->DefaultObject;
		Term->TextLiteral = Net->DefaultTextValue;
		Term->bIsLiteral = true;
		return Term;
	}

	/** Returns a UStruct scope corresponding to the pin type passed in, if one exists */
	UStruct* GetScopeFromPinType(FEdGraphPinType& Type, UClass* SelfClass)
	{
		if ((Type.PinCategory == UEdGraphSchema_K2::PC_Object) || (Type.PinCategory == UEdGraphSchema_K2::PC_Class) || (Type.PinCategory == UEdGraphSchema_K2::PC_Interface))
		{
			UClass* SubType = (Type.PinSubCategory == UEdGraphSchema_K2::PSC_Self) ? SelfClass : Cast<UClass>(Type.PinSubCategoryObject.Get());
			return SubType;
		}
		else if (Type.PinCategory == UEdGraphSchema_K2::PC_Struct)
		{
			UScriptStruct* SubType = Cast<UScriptStruct>(Type.PinSubCategoryObject.Get());
			return SubType;
		}
		else
		{
			return NULL;
		}
	}

	UBlueprint* GetBlueprint()
	{
		return Blueprint;
	}

	FBlueprintCompiledStatement& PrependStatementForNode(UEdGraphNode* Node)
	{
		FBlueprintCompiledStatement* Result = new FBlueprintCompiledStatement();
		AllGeneratedStatements.Add(Result);

		TArray<FBlueprintCompiledStatement*>& StatementList = StatementsPerNode.FindOrAdd(Node);
		StatementList.Insert(Result, 0);

		return *Result;
	}

	/** Enqueue a statement to be executed when the specified Node is triggered */
	FBlueprintCompiledStatement& AppendStatementForNode(UEdGraphNode* Node)
	{
		FBlueprintCompiledStatement* Result = new FBlueprintCompiledStatement();
		AllGeneratedStatements.Add(Result);

		TArray<FBlueprintCompiledStatement*>& StatementList = StatementsPerNode.FindOrAdd(Node);
		StatementList.Add(Result);

		return *Result;
	}

	/** Prepends the statements corresponding to Source to the set of statements corresponding to Dest */
	void CopyAndPrependStatements(UEdGraphNode* Destination, UEdGraphNode* Source)
	{
		TArray<FBlueprintCompiledStatement*>* SourceStatementList = StatementsPerNode.Find(Source);
		if (SourceStatementList)
		{
			// Mapping of jump targets for when kismet compile statements want to jump to other statements
			TMap<FBlueprintCompiledStatement*, int32> JumpTargetIndexTable;

			TArray<FBlueprintCompiledStatement*>& TargetStatementList = StatementsPerNode.FindOrAdd(Destination);

			TargetStatementList.InsertUninitialized(0, SourceStatementList->Num());
			for (int32 i = 0; i < SourceStatementList->Num(); ++i)
			{
				FBlueprintCompiledStatement* CopiedStatement = new FBlueprintCompiledStatement();
				AllGeneratedStatements.Add(CopiedStatement);

				*CopiedStatement = *((*SourceStatementList)[i]);

				TargetStatementList[i] = CopiedStatement;

				// If the statement is a jump target, keep a mapping of it so we can fix it up after this loop
				if (CopiedStatement->bIsJumpTarget)
				{
					JumpTargetIndexTable.Add((*SourceStatementList)[i], i);
				}
			}

			// Loop through all statements and remap the target labels to the mapped ones
			for (int32 i = 0; i < TargetStatementList.Num(); i++)
			{
				FBlueprintCompiledStatement* Statement = TargetStatementList[i];
				int32* JumpTargetIdx = JumpTargetIndexTable.Find(Statement->TargetLabel);
				if (JumpTargetIdx)
				{
					Statement->TargetLabel = TargetStatementList[*JumpTargetIdx];
				}
			}
		}
		else
		{
			// A node that generated no code of it's own (Source) tried to inject code into (Destination).
			// It is ok, for example: UK2Node_GetClassDefaults works like this.
			// Moreover when KismetCompilerDebugOptions::EmitNodeComments is enabled there is always a Comment state generated anyway.
		}
	}

	/** Returns true if Node generated code, and false otherwise */
	bool DidNodeGenerateCode(UEdGraphNode* Node)
	{
		TArray<FBlueprintCompiledStatement*>* SourceStatementList = StatementsPerNode.Find(Node);
		return (SourceStatementList != NULL) && (SourceStatementList->Num() > 0);
	}

private:
	// Optimize out any useless jumps (jump to the very next statement, where the control flow can just fall through)
	void MergeAdjacentStates();

	// Sorts the 'linear execution list' again by likely execution order; the list should only contain impure nodes by this point.
	void FinalSortLinearExecList();

	/** Resolves all pending goto fixups; Should only be called after all nodes have had a chance to generate code! */
	void ResolveGotoFixups();

public:
	/** Returns true if this statement cannot be optimized to remove the flow stack */
	static bool DoesStatementRequiresFlowStack(const FBlueprintCompiledStatement* Statement);
	
	/** This function links gotos, sorts statments, and merges adjacent ones */
	void ResolveStatements();

	/**
	 * Makes sure an KCST_WireTraceSite is inserted before the specified
	 * statement, and associates the specified pin with the inserted wire-trace
	 * (so we can backwards engineer which pin triggered the goto).
	 *
	 * @param  GotoStatement		The statement to insert a goto before.
	 * @param  AssociatedExecPin	The pin responsible for executing the goto.
	 */
	void InsertWireTrace(FBlueprintCompiledStatement* GotoStatement, UEdGraphPin* AssociatedExecPin)
	{
		// only need wire traces if we're debugging the blueprint is available (not for cooked builds)
		if (IsDebuggingOrInstrumentationRequired() && (AssociatedExecPin != NULL))
		{
			UEdGraphNode* PreJumpNode = AssociatedExecPin->GetOwningNode();

			TArray<FBlueprintCompiledStatement*>* NodeStatementList = StatementsPerNode.Find(PreJumpNode);
			if (NodeStatementList != NULL)
			{
				// @TODO: this Find() is potentially costly (if the node initially generated a lot of statements)
				int32 GotoIndex = NodeStatementList->Find(GotoStatement);
				if (GotoIndex != INDEX_NONE)
				{
					FBlueprintCompiledStatement* PrevStatement = NULL;
					if (GotoIndex > 0)
					{
						PrevStatement = (*NodeStatementList)[GotoIndex - 1];
					}

					// if a wire-trace has already been inserted for us
					if ((PrevStatement != NULL) && PrevStatement->Type == GetWireTraceType())
					{
						PrevStatement->ExecContext = AssociatedExecPin;
					}
					else if (PrevStatement != NULL)
					{
						FBlueprintCompiledStatement* TraceStatement = new FBlueprintCompiledStatement();
						TraceStatement->Type = GetWireTraceType();
						TraceStatement->Comment = PreJumpNode->NodeComment.IsEmpty() ? PreJumpNode->GetName() : PreJumpNode->NodeComment;
						TraceStatement->ExecContext = AssociatedExecPin;

						NodeStatementList->Insert(TraceStatement, GotoIndex);
						// AllGeneratedStatements is an unordered list, so it doesn't matter that we throw this at the end
						AllGeneratedStatements.Add(TraceStatement);
					}
				}
			}
		}
	}

	/** Looks for a pin of the given name, erroring if the pin is not found or if the direction doesn't match (doesn't verify the pin type) */
	UEdGraphPin* FindRequiredPinByName(const UEdGraphNode* Node, const FName PinName, EEdGraphPinDirection RequiredDirection = EGPD_MAX)
	{
		for (UEdGraphPin* Pin : Node->Pins)
		{
			if (Pin->PinName == PinName)
			{
				if ((Pin->Direction == RequiredDirection) || (RequiredDirection == EGPD_MAX))
				{
					return Pin;
				}
				else
				{
					MessageLog.Error(*FString::Printf(TEXT("Expected @@ to be an %s"), (RequiredDirection == EGPD_Output) ? TEXT("output") : TEXT("input")), Pin);
					return nullptr;
				}
			}
		}

		MessageLog.Error(*FString::Printf(TEXT("Expected to find a pin named %s on @@"), *PinName.ToString()), Node);
		return nullptr;
	}

	/** Checks to see if a pin is of the requested type */
	bool ValidatePinType(const UEdGraphPin* Pin, const FEdGraphPinType& TestType)
	{
		if (Pin == NULL)
		{
			// No need to error, the previous call that tried to find a pin has already errored by this point
			return false;
		}
		else
		{
			if (Pin->PinType == TestType)
			{
				return true;
			}
			else
			{
				MessageLog.Error(*FString::Printf(TEXT("Expected @@ to %s instead of %s"), *Schema->TypeToText(TestType).ToString(), *Schema->TypeToText(Pin->PinType).ToString()), Pin);
				return false;
			}
		}
	}

	KISMETCOMPILER_API FBPTerminal* CreateLocalTerminalFromPinAutoChooseScope(UEdGraphPin* Net, FString NewName);
	KISMETCOMPILER_API FBPTerminal* CreateLocalTerminal(ETerminalSpecification Spec = ETerminalSpecification::TS_Unspecified);
};

//////////////////////////////////////////////////////////////////////////
