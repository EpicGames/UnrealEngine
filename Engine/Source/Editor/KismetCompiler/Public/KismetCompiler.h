// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BPTerminal.h"
#include "Containers/Array.h"
#include "Containers/IndirectArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphCompilerUtilities.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "HAL/PlatformMath.h"
#include "K2Node_Event.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompilerMisc.h"
#include "Logging/LogMacros.h"
#include "Math/Color.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/Casts.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "Templates/TypeHash.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealNames.h"

#define UE_API KISMETCOMPILER_API

class FField;
class FKismetCompilerContext;
class FKismetCompilerVMBackend;
class FLinkerLoad;
class FProperty;
class FMulticastDelegateProperty;
class UBlueprintGeneratedClass;
class UClass;
class UEdGraph;
class UFunction;
class UK2Node_CreateDelegate;
class UK2Node_Event;
class UK2Node_FunctionEntry;
class UK2Node_TemporaryVariable;
class UK2Node_Timeline;
class UK2Node_Tunnel;
class UScriptStruct;
class UStruct;
struct FBPTerminal;
struct FKismetFunctionContext;
struct FUserPinInfo;

KISMETCOMPILER_API DECLARE_LOG_CATEGORY_EXTERN(LogK2Compiler, Log, All);

//////////////////////////////////////////////////////////////////////////
// FKismetCompilerContext

enum class EInternalCompilerFlags
{
	None = 0x0,

	PostponeLocalsGenerationUntilPhaseTwo = 0x1,
	PostponeDefaultObjectAssignmentUntilReinstancing = 0x2,
	SkipRefreshExternalBlueprintDependencyNodes = 0x4,
};
ENUM_CLASS_FLAGS(EInternalCompilerFlags)

typedef TFunction<TSharedPtr<FKismetCompilerContext>(UBlueprint*, FCompilerResultsLog&, const FKismetCompilerOptions&)> CompilerContextFactoryFunction;

class FKismetCompilerContext : public FGraphCompilerContext
{
public:

	DECLARE_EVENT_OneParam(FKismetCompilerContext, FOnFunctionListCompiled, FKismetCompilerContext*);
	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnPreCompileBlueprint, UBlueprint*);

protected:
	typedef FGraphCompilerContext Super;

	// Schema for the graph being compiled 
	UEdGraphSchema_K2* Schema;

	// Map from node class to a handler functor
	TMap< TSubclassOf<class UEdGraphNode>, FNodeHandlingFunctor*> NodeHandlers;

	// Array of function refs to run after the CDO has been compiled in FKismetCompilerContext::PostCDOCompiled	
	TArray<TFunction<void(const UObject::FPostCDOCompiledContext&, UObject*)>> PostCDOCompileSteps;

	// Map of properties created for timelines; to aid in debug data generation
	TMap<class UTimelineTemplate*, class FProperty*> TimelineToMemberVariableMap;

	// Map from UProperties to default object values, to be fixed up after compilation is complete
	TMap<FName, FString> DefaultPropertyValueMap;

	// Names of functions created
	TSet<FString> CreatedFunctionNames;

	// List of functions currently allocated
	TIndirectArray<FKismetFunctionContext> FunctionList;

	/** Set of function graphs generated for the class layout at compile time  */
	TArray<UEdGraph*> GeneratedFunctionGraphs;

	/** Set of ubergraph pages generated for the class layout at compile time  */
	TArray<UEdGraph*> GeneratedUbergraphPages;

	/**
	 * Set of generated multicast delegate properties
	 */
	TArray<FMulticastDelegateProperty*> GeneratedMulticastDelegateProps;

	/** Event that is broadcast immediately after the function list for this context has been compiled. */
	FOnFunctionListCompiled FunctionListCompiledEvent;

	// This struct holds the various compilation options, such as which passes to perform, whether to save intermediate results, etc
	FKismetCompilerOptions CompileOptions;

	// Maximum height encountered in this row; used to position the next row appropriately
	int32 MacroRowMaxHeight;

	// Maximum bounds of the spawning area
	int32 MinimumSpawnX;
	int32 MaximumSpawnX;

	// Average node size for nodes with no size
	int32 AverageNodeWidth;
	int32 AverageNodeHeight;

	// Padding
	int32 HorizontalSectionPadding;
	int32 VerticalSectionPadding;
	int32 HorizontalNodePadding;
	
	// Used to space expanded macro nodes when saving intermediate results
	int32 MacroSpawnX;
	int32 MacroSpawnY;

	UScriptStruct* VectorStruct;
	UScriptStruct* RotatorStruct;
	UScriptStruct* TransformStruct;
	UScriptStruct* LinearColorStruct;

public:
	UBlueprint* Blueprint;
	UBlueprintGeneratedClass* NewClass;
	UBlueprintGeneratedClass* OldClass;

	// The ubergraph; valid from roughly the start of CreateAndProcessEventGraph
	UEdGraph* ConsolidatedEventGraph;

	// The ubergraph context; valid from the end of CreateAndProcessEventGraph
	FKismetFunctionContext* UbergraphContext;

	TMap<UEdGraphNode*, UEdGraphNode*> CallsIntoUbergraph;
	int32 bIsFullCompile:1;

	// Map that can be used to find the macro node that spawned a provided node, 
	// if any. Macro instances can have more macros inside of them, so entries 
	// in this map may chain (i.e. values may also need to be used as keys to find
	// the full chain). Used to generate deterministic, unique identifiers for 
	// properties generated by nodes.
	TMap<UEdGraphNode*, UEdGraphNode*> MacroGeneratedNodes;

	// Map from UProperties to their RepNotify graph
	TMap<FName, FProperty*> RepNotifyFunctionMap;

	// Map from a name to the number of times it's been 'created' (identical nodes create the same variable names, so they need something appended)
	FNetNameMapping ClassScopeNetNameMap;

	// Data that persists across CompileClassLayout/CompileFunctions calls:
	UObject* OldCDO;
	int32 OldGenLinkerIdx;
	FLinkerLoad* OldLinker;
	UBlueprintGeneratedClass* TargetClass;

	// Flag to trigger FMulticastDelegateProperty::SignatureFunction resolution in 
	// CreateClassVariablesFromBlueprint:
	bool bAssignDelegateSignatureFunction;

private:
	/**
	* Set to true when there is an input event the graph and we want to automatically set up input component binding.
	*/
	bool bAutomaticallyRegisterInputEvents;

public:

	UE_API void SetAutomaticallyRegisterInputEvents(const bool bVal);
	UE_API bool GetAutomaticallyRegisterInputEvents() const;
	
	struct FDelegateInfo
	{
		FName ProxyFunctionName;
		FName CapturedVariableName;
	};

	TMap<UK2Node_CreateDelegate*, FDelegateInfo> ConvertibleDelegates;

	static UE_API FSimpleMulticastDelegate OnPreCompile;
	static UE_API FSimpleMulticastDelegate OnPostCompile;
	static UE_API FOnPreCompileBlueprint OnPreCompileBlueprint;

	/** Broadcasts a notification immediately after the function list for this context has been compiled. */
	FOnFunctionListCompiled& OnFunctionListCompiled() { return FunctionListCompiledEvent; }

	UE_API FKismetCompilerContext(UBlueprint* SourceSketch, FCompilerResultsLog& InMessageLog, const FKismetCompilerOptions& InCompilerOptions);
	UE_API virtual ~FKismetCompilerContext();
	
	/** Compile the class layout of the blueprint */
	UE_API void CompileClassLayout(EInternalCompilerFlags InternalFlags);
	
	/** Compile the functions of the blueprint - must be done after compiling the class layout: */
	UE_API void CompileFunctions(EInternalCompilerFlags InternalFlags);

	/** Called after the CDO has been generated, allows assignment of cached/derived data: */
	UE_API void PostCDOCompiled(const UObject::FPostCDOCompiledContext& Context);

	/**
	* Adds the given "StepFunction" to be run after the CDO has been compiled in FKismetCompilerContext::PostCDOCompiled.
	*/
	UE_API void AddPostCDOCompiledStep(TFunction<void (const UObject::FPostCDOCompiledContext&, UObject*)>&& StepFunction);

	/** Compile a blueprint into a class and a set of functions */
	UE_API void Compile();

	/** Function used to assign the new class that will be used by the compiler */
	UE_API void SetNewClass(UBlueprintGeneratedClass* ClassToUse);

	const UEdGraphSchema_K2* GetSchema() const { return Schema; }

	/** Spawn an intermediate function graph for this compilation using the specified desired name (and optional signature),
		which may be modified to make it unique. */
	UE_API UEdGraph* SpawnIntermediateFunctionGraph(const FString& InDesiredFunctionName, const UFunction* InSignature = nullptr, bool bUseUniqueName = true);

	// Spawns an intermediate node associated with the source node (for error purposes)
	template <typename NodeType>
	NodeType* SpawnIntermediateNode(UEdGraphNode* SourceNode, UEdGraph* ParentGraph = nullptr)
	{
		if (ParentGraph == nullptr)
		{
			ParentGraph = SourceNode->GetGraph();
		}

		NodeType* Result = ParentGraph->CreateIntermediateNode<NodeType>();
		MessageLog.NotifyIntermediateObjectCreation(Result, SourceNode); // this might be useful to track back function entry nodes to events.
		Result->CreateDeterministicGuid();

		AutoAssignNodePosition(Result);

		return Result;
	}

	/**
	 * Moves pin links over from the source-pin to the specified intermediate, 
	 * and validates the result (additionally logs a redirect from the 
	 * intermediate-pin back to the source so we can back trace for debugging, etc.)
	 * 
	 * @param  SourcePin		The pin you want disconnected.
	 * @param  IntermediatePin	The pin you want the SourcePin's links moved to.
	 * @return The result from calling the schema's MovePinLinks().
	 */
	UE_API FPinConnectionResponse MovePinLinksToIntermediate(UEdGraphPin& SourcePin, UEdGraphPin& IntermediatePin);

	/**
	 * Copies pin links over from the source-pin to the specified intermediate, 
	 * and validates the result (additionally logs a redirect from the 
	 * intermediate-pin back to the source so we can back trace for debugging, etc.)
	 * 
	 * @param  SourcePin		The pin whose links you want copied.
	 * @param  IntermediatePin	The pin you want the SourcePin's links copied to.
	 * @return The result from calling the schema's CopyPinLinks().
	 */
	UE_API FPinConnectionResponse CopyPinLinksToIntermediate(UEdGraphPin& SourcePin, UEdGraphPin& IntermediatePin);

	struct FNameParameterHelper
	{
		FNameParameterHelper(const FName InNameParameter) : NameParameter(InNameParameter) { }
		FNameParameterHelper(const FString& InNameParameter) : NameParameter(*InNameParameter) { }
		FNameParameterHelper(const TCHAR* InNameParameter) : NameParameter(InNameParameter) { }

		FName operator*() const { return NameParameter; }

	private:
		FName NameParameter;
	};

	UE_API UK2Node_TemporaryVariable* SpawnInternalVariable(UEdGraphNode* SourceNode, FName Category, FName SubCategory = NAME_None, UObject* SubcategoryObject = nullptr, EPinContainerType PinContainerType = EPinContainerType::None, const FEdGraphTerminalType& ValueTerminalType = FEdGraphTerminalType());

	UE_API bool UsePersistentUberGraphFrame() const;

	UE_API FString GetGuid(const UEdGraphNode* Node) const;

	static UE_API TSharedPtr<FKismetCompilerContext> GetCompilerForBP(UBlueprint* BP, FCompilerResultsLog& InMessageLog, const FKismetCompilerOptions& InCompileOptions);
	static UE_API void RegisterCompilerForBP(UClass* BPClass, CompilerContextFactoryFunction FactoryFunction );

	/** Ensures that all variables have valid names for compilation/replication */
	UE_API void ValidateVariableNames();

	/** Ensures that all component class overrides are legal overrides of the parent class */
	UE_API void ValidateComponentClassOverrides();

	/**
	* Ensures that all class reference Properties are legal overrides of the parent class
	* by checking the default value set on any PC_Class variable types. Requires a 
	* valid CDO in order to do this validation. Called in Stage V: Validate 
	*/
	UE_API void ValidateClassPropertyDefaults();

	/** Resets the blueprint's GeneratedVariables list and calls PopulateBlueprintGeneratedVariables */
	UE_API void ResetAndPopulateBlueprintGeneratedVariables();

	/** Creates a class variable for each entry in the Blueprint NewVars array */
	UE_API virtual void CreateClassVariablesFromBlueprint();

	/**
	 * Picks the name to use for an autogenerated event stub
	 */ 
	UE_API FName GetEventStubFunctionName(UK2Node_Event* SrcEventNode);

	/**
	 * Searches the function graphs and ubergraph pages for any delegate proxies,
	 * which are then registered with the compiler context.
	 * If a "captured" variable is needed, then a new property will be added to the current class.
	 * In this context, a captured variable is any target actor that the delegate will be called on.
	 */
	UE_API void RegisterClassDelegateProxiesFromBlueprint();

protected:
	UE_API virtual UEdGraphSchema_K2* CreateSchema();
	UE_API virtual void PostCreateSchema();
	UE_API virtual void SpawnNewClass(const FString& NewClassName);
	virtual void OnNewClassSet(UBlueprintGeneratedClass* ClassToUse) {}
	virtual void OnPostCDOCompiled(const UObject::FPostCDOCompiledContext& Context) {}

	/**
	 * Backwards Compatability:  Ensures that the passed in TargetClass is of the proper type (e.g. BlueprintGeneratedClass, AnimBlueprintGeneratedClass), and NULLs the reference if it is not 
	 */
	UE_API virtual void EnsureProperGeneratedClass(UClass*& TargetClass);

	/**
	 * Removes the properties and functions from a class, so that new ones can be created in its place
	 * 
	 * @param ClassToClean		The UClass to scrub
	 * @param OldCDO			Reference to the old CDO of the class, so we can copy the properties from it to the new class's CDO
	 */
	UE_API virtual void CleanAndSanitizeClass(UBlueprintGeneratedClass* ClassToClean, UObject*& OldCDO);
	
	/** Compilers are expected to populate the blueprint's full list of GeneratedVariables here as the list is reset at this point */
	virtual void PopulateBlueprintGeneratedVariables() {}

	struct FSubobjectCollection
	{
	private:
		TSet<const UObject*> Collection;

	public:
		UE_API void AddObject(const UObject* const InObject);

		template<typename TOBJ>
		void AddObjects(const TArray<TOBJ>& InObjects)
		{
			for ( const auto& ObjPtr : InObjects )
			{
				AddObject(ObjPtr);
			}
		}

		UE_API bool operator()(const UObject* const RemovalCandidate) const;
	};

	/**
	 * Saves any SubObjects on the blueprint that need to survive the clean 
	 */
	UE_API virtual void SaveSubObjectsFromCleanAndSanitizeClass(FSubobjectCollection& SubObjectsToSave, UBlueprintGeneratedClass* ClassToClean);

	/** 
	 * Checks a connection response, and errors if it didn't succeed (not public, 
	 * users should be using MovePinLinksToIntermediate/CopyPinLinksToIntermediate 
	 * instead of wrapping their own with this).
	 */
	UE_API void CheckConnectionResponse(const FPinConnectionResponse &Response, const UEdGraphNode *Node);

	/** Prune isolated nodes given the specified graph */
	UE_API void PruneIsolatedNodes(UEdGraph* InGraph, bool bInIncludeNodesThatCouldBeExpandedToRootSet);

	// FGraphCompilerContext interface
	UE_API virtual void ValidateLink(const UEdGraphPin* PinA, const UEdGraphPin* PinB) const override;
	UE_API virtual void ValidatePin(const UEdGraphPin* Pin) const override;
	UE_API virtual void ValidateNode(const UEdGraphNode* Node) const override;
	UE_API virtual bool CanIgnoreNode(const UEdGraphNode* Node) const override;
	UE_API virtual bool ShouldForceKeepNode(const UEdGraphNode* Node) const override;
	UE_API virtual void PruneIsolatedNodes(const TArray<UEdGraphNode*>& RootSet, TArray<UEdGraphNode*>& GraphNodes) override;
	virtual bool PinIsImportantForDependancies(const UEdGraphPin* Pin) const override
	{
		// The execution wires do not form data dependencies, they are only important for final scheduling and that is handled thru gotos
		return Pin->PinType.PinCategory != UEdGraphSchema_K2::PC_Exec;
	}

	// Expands out nodes that need it
	UE_API void ExpansionStep(UEdGraph* Graph, bool bAllowUbergraphExpansions);

	// Advances the macro position tracking
	UE_API void AdvanceMacroPlacement(int32 Width, int32 Height);
	UE_API void AutoAssignNodePosition(UEdGraphNode* Node);
	UE_API void CreateCommentBlockAroundNodes(const TArray<UEdGraphNode*>& Nodes, UObject* SourceObject, UEdGraph* TargetGraph, FString CommentText, FLinearColor CommentColor, int32& Out_OffsetX, int32& Out_OffsetY);

	/** Creates a class variable */
	UE_API FProperty* CreateVariable(const FName Name, const FEdGraphPinType& Type);

	/** 
	 * Creates a multicast delegate variable & associated signature graph / function, use to generate events 
	 * 
	 * You must add an ubergraph page containing an 'UK2Node_GeneratedBoundEvent' that corresponds to this delegate
	 * to 'GeneratedUbergraphPages' in your 'FKismetCompilerContext'. We do not create this graph / page for you so that
	 * graph creation is not coupled to variable creation. We will associate this delegate with your node for you by name.
	 */
	UE_API FMulticastDelegateProperty* CreateMulticastDelegateVariable(const FName Name, const FEdGraphPinType& Type);

	/** 
	 * Creates a multicast delegate variable & associated signature graph / Function, use to generate events 
	 * 
	 * Defaults to base MulticastDelegate type. See above version with 'Type' arg for more info.
	 */
	UE_API FMulticastDelegateProperty* CreateMulticastDelegateVariable(const FName Name);

	// Gives derived classes a chance to emit debug data
	virtual void PostCompileDiagnostics() {}

	// Gives derived classes a chance to hook up any custom logic
	virtual void PreCompile() { OnPreCompile.Broadcast(); OnPreCompileBlueprint.Broadcast(Blueprint); }
	virtual void PostCompile() { OnPostCompile.Broadcast(); }

	// Gives derived classes a chance to process post-node expansion
	virtual void PostExpansionStep(const UEdGraph* Graph) {}

	/** Determines if a node is pure */
	UE_API virtual bool IsNodePure(const UEdGraphNode* Node) const;

	/** Creates a property with flags including PropertyFlags in the Scope structure for each entry in the Terms array */
	UE_API void CreatePropertiesFromList(UStruct* Scope, FField::FLinkedListBuilder& PropertyListBuilder, TIndirectArray<FBPTerminal>& Terms, EPropertyFlags PropertyFlags, bool bPropertiesAreLocal, bool bPropertiesAreParameters = false);

	/** Create the properties on a function for input/output parameters */
	UE_API void CreateParametersForFunction(FKismetFunctionContext& Context, UFunction* ParameterSignature, FField::FLinkedListBuilder& PropertyListBuilder);

	/** Creates the properties on a function that store the local and event graph (if applicable) variables */
	UE_API void CreateLocalVariablesForFunction(FKismetFunctionContext& Context, FField::FLinkedListBuilder& PropertyListBuilder);

	/** Creates user defined local variables for function */
	UE_API void CreateUserDefinedLocalVariablesForFunction(FKismetFunctionContext& Context, FField::FLinkedListBuilder& PropertyListBuilder);

	/** Helper function for CreateUserDefinedLocalVariablesForFunction and compilation manager's FastGenerateSkeletonClass: */
	static UE_API FProperty* CreateUserDefinedLocalVariableForFunction(const FBPVariableDescription& Variable, UFunction* Function, UBlueprintGeneratedClass* OwningClass, FField::FLinkedListBuilder& PropertyListBuilder, const UEdGraphSchema_K2* Schema, FCompilerResultsLog& MessageLog);

	/** Adds a default value entry into the DefaultPropertyValueMap for the property specified */
	UE_API void SetPropertyDefaultValue(const FProperty* PropertyToSet, FString& Value);

	/** Copies default values cached for the terms in the DefaultPropertyValueMap to the final CDO */
	UE_API virtual void CopyTermDefaultsToDefaultObject(UObject* DefaultObject);

	/** Non virtual wrapper to encapsulate functions that occur when the CDO is ready for values: */
	UE_API void PropagateValuesToCDO(UObject* NewCDO, UObject* OldCDO);

	/** 
	 * Function works only if subclass of AActor or UActorComponent.
	 * If ReceiveTick event is defined, force CanEverTick.
	 */
	UE_API void SetCanEverTick() const;

	/** Scan FunctionList and return Entry point, for matching one  */
	UE_API const UK2Node_FunctionEntry* FindLocalEntryPoint(const UFunction* Function) const;

	//@TODO: Debug printing
	UE_API void PrintVerboseInfoStruct(UStruct* Struct) const;
	UE_API void PrintVerboseInformation(UClass* Class) const;
	//@ENDTODO

	/**
	 * Performs transformations on specific nodes that require it according to the schema
	 */
	UE_API virtual void TransformNodes(FKismetFunctionContext& Context);

	/**
	 * Merges in any all ubergraph pages into the gathering ubergraph
	 */
	UE_API virtual void MergeUbergraphPagesIn(UEdGraph* Ubergraph);

	/**
	 * Creates a list of functions to compile
	 */
	UE_API virtual void CreateFunctionList();

	/** Creates a new function context and adds it to the function list to be processed. */
	UE_API FKismetFunctionContext* CreateFunctionContext();

	/**
	 * Merges a single ubergraph page into the main ubergraph
	 */
	UE_API virtual void MergeGraphIntoUbergraph(UEdGraph* SourceGraph, UEdGraph* Ubergraph);

	/**
	 * Merges macros/subgraphs into the graph and validates it, creating a function list entry if it's reasonable.
	 */
	UE_API virtual void ProcessOneFunctionGraph(UEdGraph* SourceGraph, bool bInternalFunction = false);

	/**
	 * Gets the unique name for this context's ExecuteUbergraph function
	 */
	FName GetUbergraphCallName() const
	{
		const FString UbergraphCallString = UEdGraphSchema_K2::FN_ExecuteUbergraphBase.ToString() + TEXT("_") + Blueprint->GetName();
		return FName(*UbergraphCallString);
	}

	/**
	 * Expands any macro instances and collapses any tunnels in the nodes of SourceGraph
	 */
	UE_API void ExpandTunnelsAndMacros(UEdGraph* SourceGraph);

	/**
	 * Maps the nodes in an intermediate tunnel expansion path back to the owning tunnel instance node.
	 */
	UE_API void MapExpansionPathToTunnelInstance(const UEdGraphNode* InnerExpansionNode, const UEdGraphNode* OuterTunnelInstance);

	/**
	* Processes an intermediate tunnel expansion boundary.
	*
	* We define a tunnel boundary as the input and output sides of an intermediate tunnel instance node expansion. Each boundary
	* consists of a pair of tunnel nodes (input/output), with one side being the tunnel "instance" node that owns the expansion.
	* After expansion, tunnel nodes are cropped and removed from the function graph, so they do not result in any actual bytecode.
	*
	* This function maps the nodes in the execution path through the expansion and back to the outer tunnel instance node. If
	* Blueprint debugging is enabled, this function also spawns one or more intermediate "boundary" NOPs around the tunnel I/O
	* pair. The boundary nodes are intended to serve as debug sites, allowing breakpoints to be hit on both sides of the tunnel.
	*
	* @param	TunnelInput		Tunnel input node. This will either be a tunnel instance node (OutputSource) or a tunnel exit node.
	* @param	TunnelOutput	Tunnel output node. This will either be a tunnel entry node or a tunnel instance node (InputSink).
	*/
	UE_API void ProcessIntermediateTunnelBoundary(UK2Node_Tunnel* TunnelInput, UK2Node_Tunnel* TunnelOutput);

	/**
	 * Merges pages and creates function stubs, etc...
	 */
	UE_API void CreateAndProcessUbergraph();

	/** Create a stub function graph for the event node, and have it invoke the correct point in the ubergraph */
	UE_API void CreateFunctionStubForEvent(UK2Node_Event* Event, UObject* OwnerOfTemporaries);

	/** Expand timeline nodes into necessary nodes */
	UE_API void ExpandTimelineNodes(UEdGraph* SourceGraph);

	/**
	 * First phase of compiling a function graph
	 *   - Performs initial validation that the graph is at least well formed enough to be processed further
	 *   - Creates a copy of the graph to allow further transformations to occur
	 *   - Prunes the 'graph' to only included the connected portion that contains the function entry point 
	 *   - Schedules execution of each node based on data/execution dependencies
	 *   - Creates a UFunction object containing parameters and local variables (but no script code yet)
	 */
	UE_API virtual void PrecompileFunction(FKismetFunctionContext& Context, EInternalCompilerFlags InternalFlags);

	/** Called to initialize generated event nodes that came from generated ubergraph pages after delegate signature compilation is done */
	UE_API virtual void InitializeGeneratedEventNodes(EInternalCompilerFlags InternalFlags);

	/**
	 * Used for performing custom patching during stage IX of the compilation during load.
	 */
	UE_API virtual void PreCompileUpdateBlueprintOnLoad(UBlueprint* BP);

	/**
	 * Second phase of compiling a function graph
	 *   - Generates an executable statement list
	 */
	UE_API virtual void CompileFunction(FKismetFunctionContext& Context);

	/**
	 * Final phase of compiling a function graph; called after all functions have had CompileFunction called
	 *   - Patches up cross-references, etc..., and performs final validation
	 */
	UE_API virtual void PostcompileFunction(FKismetFunctionContext& Context);

	/**
	 * Handles final post-compilation setup, flags, creates cached values that would normally be set during deserialization, etc...
	 */
	UE_API void FinishCompilingFunction(FKismetFunctionContext& Context);

	/** Adds metadata for a particular compiled function based on its characteristics */
	UE_API virtual void SetCalculatedMetaDataAndFlags(UFunction* Function, UK2Node_FunctionEntry* EntryNode, const UEdGraphSchema_K2* Schema );

	/** Reflects each pin's user set, default value into the function's metadata (so it can be queried for later by CallFunction nodes, etc.) */
	static UE_API void SetDefaultInputValueMetaData(UFunction* Function, const TArray< TSharedPtr<FUserPinInfo> >& InputData);

	/**
	 * Handles adding the implemented interface information to the class
	 */
	UE_API virtual void AddInterfacesFromBlueprint(UClass* Class);

	/**
	 * Handles final post-compilation setup, flags, creates cached values that would normally be set during deserialization, etc...
	 */
	UE_API virtual void FinishCompilingClass(UClass* Class);

	/** Build the dynamic bindings objects used to tie events to delegates at runtime */
	UE_API void BuildDynamicBindingObjects(UBlueprintGeneratedClass* Class);

	/**
	 *  If a function in the graph cannot be placed as event make sure that it is not.
	 */
	UE_API void VerifyValidOverrideEvent(const UEdGraph* Graph);

	/**
	 *  If a function in the graph cannot be overridden make sure that it is not.
	 */
	UE_API void VerifyValidOverrideFunction(const UEdGraph* Graph);

	/**
	 * Checks if self pins are connected.
	 */
	UE_API void ValidateSelfPinsInGraph(FKismetFunctionContext& Context);

	/**
	* Checks if pin types are unresolved (e.g. still wildcards).
	*/
	UE_API void ValidateNoWildcardPinsInGraph(const UEdGraph* SourceGraph);

	/** Ensures that all timelines have valid names for compilation/replication */
	UE_API void ValidateTimelineNames();

	/** Ensures that all function graphs have valid names for compilation/replication */
	UE_API void ValidateFunctionGraphNames();

	/** Validates the generated class */
	UE_API virtual bool ValidateGeneratedClass(UBlueprintGeneratedClass* Class);

	/** Discovers exec pin links for the sourcenode */
	UE_API void DetermineNodeExecLinks(UEdGraphNode* SourceNode, TMap<UEdGraphPin*, UEdGraphPin*>& SourceNodeLinks) const;

private:
	UE_API void CreateLocalsAndRegisterNets(FKismetFunctionContext& Context, FField::FLinkedListBuilder& PropertyListBuilder);

	/**
	 * Handles creating a new event node for a given output on a timeline node utilizing the named function
	 */
	UE_API void CreatePinEventNodeForTimelineFunction(UK2Node_Timeline* TimelineNode, UEdGraph* SourceGraph, FName FunctionName, const FName PinName, FName ExecFuncName);

	/** Util for creating a node to call a function on a timeline and move connections to it */
	UE_API class UK2Node_CallFunction* CreateCallTimelineFunction(UK2Node_Timeline* TimelineNode, UEdGraph* SourceGraph, FName FunctionName, UEdGraphPin* TimelineVarPin, UEdGraphPin* TimelineFunctionPin);

	/**
	 * Function to reset graph node's error flag before compiling
	 *
	 * @param: Reference to graph instance
	 */
	UE_API void ResetErrorFlags(UEdGraph* Graph) const;
	
	/**
	 * Registers any nodes that are bound to a "convertible" function signature.
	 * Specifically, these are function signatures that differ only by float/double parameters.
	 * 
	 * @param: Reference to graph instance
	 */
	UE_API void RegisterConvertibleDelegates(UEdGraph* Graph);

	/**
	 * Modifies the graph to use a proxy delegate function if it uses a convertible delegate signature.
	 * This involves several steps:
	 * 1. Creates a new function graph that uses the exact function signature of the delegate.
	 * 2. Adds and links the original delegate function call, which implicity casts the input parameters.
	 * 3. If applicable, adds a node to the original graph that sets the variable of the target actor (ie: the captured variable)
	 * 
	 * @param: Reference to graph instance
	 */
	UE_API void ReplaceConvertibleDelegates(UEdGraph* Graph);

	/**
	 * The compilation manager is a new client of the compiler - it reuses several virtual functions
	 * that are not usefully public in any other context and so rather than expand the public interface
	 * we have decided to make the compilation manager a friend:
	 */
	friend struct FBlueprintCompilationManagerImpl;
	friend struct FRigVMBlueprintCompilationManagerImpl;
};

//////////////////////////////////////////////////////////////////////////

#undef UE_API
