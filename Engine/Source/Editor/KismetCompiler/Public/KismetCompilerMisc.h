// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BPTerminal.h"
#include "BlueprintCompiledStatement.h"
#include "Containers/Array.h"
#include "Containers/IndirectArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"

#define UE_API KISMETCOMPILER_API

class FCompilerResultsLog;
class FKismetCompilerContext;
class FProperty;
class UBlueprint;
class UClass;
class UEdGraph;
class UEdGraphNode;
class UEdGraphPin;
class UEdGraphSchema_K2;
class UFunction;
class UK2Node;
class UK2Node_CallFunction;
class UObject;
class UStruct;
struct FBPTerminal;
struct FEdGraphPinType;
struct FKismetFunctionContext;
struct FMemberReference;

//////////////////////////////////////////////////////////////////////////
// FKismetCompilerUtilities

// Used by DoSignaturesHaveConvertibleFloatTypes
enum class ConvertibleSignatureMatchResult
{
	ExactMatch,					// The function signatures are an exact match
	HasConvertibleFloatParams,	// The function signatures are identical, except for float/double mismatches, which can be converted
	Different					// The function signatures are completely different
};

/** This is a loose collection of utilities used when 'compiling' a new UClass from a K2 graph. */
class FKismetCompilerUtilities
{
public:
	// Rename a class and it's CDO into the transient package, and clear RF_Public on both of them
	static UE_API void ConsignToOblivion(UClass* OldClass, bool bForceNoResetLoaders);


	static UE_API void UpdateBlueprintSkeletonStubClassAfterFailedCompile(UBlueprint* Blueprint, UClass* StubClass);

	/**
	 * Tests to see if a pin is schema compatible with a property.
	 *
	 * @param	SourcePin		If non-null, source object.
	 * @param	Property		The property to check.
	 * @param	MessageLog  	The message log.
	 * @param	Schema			Schema.
	 * @param	SelfClass   	Self class (needed for pins marked Self).
	 *
	 * @return	true if the pin type/direction is compatible with the property.
	 */
	static UE_API bool IsTypeCompatibleWithProperty(UEdGraphPin* SourcePin, FProperty* Property, FCompilerResultsLog& MessageLog, const UEdGraphSchema_K2* Schema, UClass* SelfClass);

	/** Finds a property by name, starting in the specified scope; Validates property type and returns NULL along with emitting an error if there is a mismatch. */
	static UE_API FProperty* FindPropertyInScope(UStruct* Scope, UEdGraphPin* Pin, FCompilerResultsLog& MessageLog, const UEdGraphSchema_K2* Schema, UClass* SelfClass, bool& bIsSparseProperty);

	// Finds a property by name, starting in the specified scope, returning NULL if it's not found
	static UE_API FProperty* FindNamedPropertyInScope(UStruct* Scope, FName PropertyName, bool& bIsSparseProperty, const bool bAllowDeprecated = false);

	/** return function, that overrides BlueprintImplementableEvent with given name in given class (super-classes are not considered) */
	static UE_API const UFunction* FindOverriddenImplementableEvent(const FName& EventName, const UClass* Class);

	/** Helper function for creating property for primitive types. Used only to create inner peroperties for FArrayProperty, FSetProperty, and FMapProperty: */
	static UE_API FProperty* CreatePrimitiveProperty( FFieldVariant PropertyScope, const FName& ValidatedPropertyName, const FName& PinCategory, const FName& PinSubCategory, UObject* PinSubCategoryObject, UClass* SelfClass, bool bIsWeakPointer, const class UEdGraphSchema_K2* Schema, FCompilerResultsLog& MessageLog);

	/** Creates a property named PropertyName of type PropertyType in the Scope or returns NULL if the type is unknown, but does *not* link that property in */
	static UE_API FProperty* CreatePropertyOnScope(UStruct* Scope, const FName& PropertyName, const FEdGraphPinType& Type, UClass* SelfClass, EPropertyFlags PropertyFlags, const class UEdGraphSchema_K2* Schema, FCompilerResultsLog& MessageLog, UEdGraphPin* SourcePin = nullptr);

	/**
	 * Checks that the property name isn't taken in the given scope (used by CreatePropertyOnScope())
	 *
	 * @return	Ptr to an existing object with that name in the given scope or nullptr if none exists
	 */
	static UE_API FFieldVariant CheckPropertyNameOnScope(UStruct* Scope, const FName& PropertyName);

	// Find groups of nodes, that can be executed separately.
	static UE_API TArray<TSet<UEdGraphNode*>> FindUnsortedSeparateExecutionGroups(const TArray<UEdGraphNode*>& Nodes);

private:
	/** Counter to ensure unique names in the transient package, to avoid GC collection issues with classes and their CDOs */
	static uint32 ConsignToOblivionCounter;
public:
	static UE_API void CompileDefaultProperties(UClass* Class);

	static UE_API void LinkAddedProperty(UStruct* Structure, FProperty* NewProperty);

	static UE_API void RemoveObjectRedirectorIfPresent(UObject* Package, const FString& ClassName, UObject* ObjectBeingMovedIn);

	/* checks if enum variables from given object store proper indexes */
	static UE_API void ValidateEnumProperties(const UObject* DefaultObject, FCompilerResultsLog& MessageLog);

	/** checks if the specified pin can default to self */
	static UE_API bool ValidateSelfCompatibility(const UEdGraphPin* Pin, FKismetFunctionContext& Context);

	/** Create 'set var by name' nodes and hook them up - used to set values when components are added or actor are created at run time. Returns the 'last then' pin of the assignment nodes */
	static UE_API UEdGraphPin* GenerateAssignmentNodes(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph, UK2Node* CallBeginSpawnNode, UEdGraphNode* SpawnNode, UEdGraphPin* CallBeginResult, const UClass* ForClass, const UEdGraphPin* CallBeginClassInput = nullptr);

	/** Create node that replace regular setter and use the SetPropertyValueAndBroadcast. */
	static UE_API TTuple<UEdGraphPin*, UEdGraphPin*> GenerateFieldNotificationSetNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph, UEdGraphNode* SourceNode, UEdGraphPin* SelfPin, FProperty* VariableProperty, const FMemberReference& VariableReference, bool bHasLocalRepNotify, bool bShouldFlushDormancyOnSet, bool bIsNetProperty);
	
	/** Create node to broadcast a FieldNotification value changed */
	static UE_API TTuple<UEdGraphPin*, UEdGraphPin*> GenerateBroadcastFieldNotificationNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph, UEdGraphNode* SourceNode, FProperty* Property);

	/** Create Kismet assignment statement with proper object <-> interface cast */
	static UE_API void CreateObjectAssignmentStatement(FKismetFunctionContext& Context, UEdGraphNode* Node, FBPTerminal* SrcTerm, FBPTerminal* DstTerm, UEdGraphPin* DstPin = nullptr);

	UE_DEPRECATED(5.4, "ValidateProperEndExecutionPath is deprecated.")
	static UE_API void ValidateProperEndExecutionPath(FKismetFunctionContext& Context);

	/** Generate an error for non-const output parameters */
	static UE_API void DetectValuesReturnedByRef(const UFunction* Func, const UK2Node * Node, FCompilerResultsLog& MessageLog);

	/** @return true when the property is a BP user variable that should uses UFieldNotificationLibrary::SetPropertyValueAndBroadcast to set its value. */
	static UE_API bool IsPropertyUsesFieldNotificationSetValueAndBroadcast(const FProperty* Property);

	static UE_API bool IsStatementReducible(EKismetCompiledStatementType StatementType);

	/**
	 * Intended to avoid errors that come from checking for external member 
	 * (function, variable, etc.) dependencies. This can happen when a Blueprint
	 * was saved without having new members compiled in (saving w/out compiling),
	 * and a Blueprint that uses those members is compiled-on-load before the 
	 * uncompiled one. Any valid errors should surface later, when the dependant 
	 * Blueprint's bytecode is recompiled.
	 */
	static UE_API bool IsMissingMemberPotentiallyLoading(const UBlueprint* SelfBlueprint, const UStruct* MemberOwner);

	/** @return true if the graph in question contains only an entry node or only an entry node and a call to its parent if the graph is an override */
	static UE_API bool IsIntermediateFunctionGraphTrivial(FName FunctionName, const UEdGraph* FunctionGraph);

	/** Add this BP to any BPs that it in*/
	static UE_API void UpdateDependentBlueprints(UBlueprint* BP);

	// Check the passed-in function to verify its thread safety. This makes sure that it only uses/calls thread-safe functions/nodes.
	static UE_API bool CheckFunctionThreadSafety(const FKismetFunctionContext& InContext, FCompilerResultsLog& InMessageLog, bool InbEmitErrors = true);

	// Helper function used by CheckFunctionThreadSafety. Split out to allow the ability to examine individual compiled statement lists (e.g. for the ubergraph)
	static UE_API bool CheckFunctionCompiledStatementsThreadSafety(const UEdGraphNode* InNode, const UEdGraph* InSourceGraph, const TArray<FBlueprintCompiledStatement*>& InStatements, FCompilerResultsLog& InMessageLog, bool InbEmitErrors = true, TSet<const FBPTerminal*>* InThreadSafeObjectTerms = nullptr);

	/** Similar to UFunction::IsSignatureCompatibleWith, but also checks if the function signatures are convertible.
	 *
	 * For example, if a parameter in SourceFunction is of type float, but its corresponding type in OtherFunction is double, then the function is deemed "convertible".
	 * This is primarily used for binding Blueprint functions with native delegate signatures that use float types.
	 */
	static UE_API ConvertibleSignatureMatchResult DoSignaturesHaveConvertibleFloatTypes(const UFunction* SourceFunction, const UFunction* OtherFunction);
};

//////////////////////////////////////////////////////////////////////////
// FNodeHandlingFunctor

class FNodeHandlingFunctor
{
public:
	class FKismetCompilerContext& CompilerContext;

protected:
	/** Helper function that verifies the variable name referenced by the net exists in the associated scope (either the class being compiled or via an object reference on the Self pin), and then creates/registers a term for that variable access. */
	UE_API void ResolveAndRegisterScopedTerm(FKismetFunctionContext& Context, UEdGraphPin* Net, TIndirectArray<FBPTerminal>& NetArray);

	// Generate a goto on the corresponding exec pin
	UE_API FBlueprintCompiledStatement& GenerateSimpleThenGoto(FKismetFunctionContext& Context, UEdGraphNode& Node, UEdGraphPin* ThenExecPin);

	// Generate a goto corresponding to the then pin(s)
	UE_API FBlueprintCompiledStatement& GenerateSimpleThenGoto(FKismetFunctionContext& Context, UEdGraphNode& Node);

	// If the net is a literal, it validates the default value and registers it.
	// Returns true if the net is *not* a literal, or if it's a literal that is valid.
	// Returns false only for a bogus literal value.
	UE_API bool ValidateAndRegisterNetIfLiteral(FKismetFunctionContext& Context, UEdGraphPin* Net);

	// Helper to register literal term
	UE_API virtual FBPTerminal* RegisterLiteral(FKismetFunctionContext& Context, UEdGraphPin* Net);
public:
	FNodeHandlingFunctor(FKismetCompilerContext& InCompilerContext)
		: CompilerContext(InCompilerContext)
	{
	}

	virtual ~FNodeHandlingFunctor() = default; 
	
	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node)
	{
	}

	virtual void Transform(FKismetFunctionContext& Context, UEdGraphNode* Node)
	{
	}

	virtual void RegisterNet(FKismetFunctionContext& Context, UEdGraphPin* Pin)
	{
	}

	UE_API virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node);

	// Returns true if this kind of node needs to be handled in the first pass, prior to execution scheduling (this is only used for function entry and return nodes)
	virtual bool RequiresRegisterNetsBeforeScheduling() const
	{
		return false;
	}

	/**
	 * Creates a sanitized name.
	 *
	 * @param [in,out]	Name	The name to modify and make a legal C++ identifier.
	 */
	static UE_API void SanitizeName(FString& Name);
};

class FKCHandler_Passthru : public FNodeHandlingFunctor
{
public:
	FKCHandler_Passthru(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{
	}

	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node)
	{
		// Generate the output impulse from this node
		GenerateSimpleThenGoto(Context, *Node);
	}
};


//////////////////////////////////////////////////////////////////////////
// FNetNameMapping

// Map from a name to the number of times it's been 'created' (identical nodes create the same variable names, so they need something appended)
struct FNetNameMapping
{
public:
	// Come up with a valid, unique (within the scope of NetNameMap) name based on an existing Net object and (optional) context.
	// The resulting name is stable across multiple calls if given the same pointer.
	FString MakeValidName(const UEdGraphNode* Net, const FString& Context = TEXT("")) { return MakeValidNameImpl(Net, Context); }
	FString MakeValidName(const UEdGraphPin* Net, const FString& Context = TEXT("")) { return MakeValidNameImpl(Net, Context); }
	FString MakeValidName(const UObject* Net, const FString& Context = TEXT("")) { return MakeValidNameImpl(Net, Context); }

private:
	KISMETCOMPILER_API static FString MakeBaseName(const UEdGraphNode* Net);
	KISMETCOMPILER_API static FString MakeBaseName(const UEdGraphPin* Net);
	KISMETCOMPILER_API static FString MakeBaseName(const UObject* Net);

	template< typename NetType >
	FString MakeValidNameImpl(NetType Net, const FString& Context)
	{
		// Check to see if this net was already used to generate a name
		if (FString* Result = NetToName.Find(Net))
		{
			return *Result;
		}
		else
		{
			FString BaseName = MakeBaseName(Net);
			if (!Context.IsEmpty())
			{
				BaseName += FString::Printf(TEXT("_%s"), *Context);
			}

			FString NetName = GetUniqueName(MoveTemp(BaseName));

			NetToName.Add(Net, NetName);
			NameToNet.Add(NetName, Net);
			return NetName;
		}
	}

	FString GetUniqueName(FString NetName)
	{
		FNodeHandlingFunctor::SanitizeName(NetName);
		// Scratchpad so that we can find a new postfix:
		FString NewNetName(NetName);

		int32 Postfix = 0;
		const void** ExistingNet = NameToNet.Find(NewNetName);
		while(ExistingNet && *ExistingNet)
		{
			++Postfix;
			// Add an integer to the base name and check if it's free:
			NewNetName = NetName + TEXT("_") + FString::FromInt(Postfix);
			ExistingNet = NameToNet.Find(NewNetName);
		}
		return NewNetName;
	}
	
	TMap<const void*, FString> NetToName;
	TMap<FString, const void*> NameToNet;
};

#undef UE_API
