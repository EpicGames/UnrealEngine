// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SubclassOf.h"

class FBlueprintCompileReinstancer;
class FCompilerResultsLog;
class UBlueprint;
class UBlueprintGeneratedClass;
class UClass;
class UUserDefinedEnum;
class UUserDefinedStruct;
struct FKismetCompilerOptions;

#define KISMET_COMPILER_MODULENAME "KismetCompiler"

//////////////////////////////////////////////////////////////////////////
// IKismetCompilerInterface

class IBlueprintCompiler
{
public:
	virtual void PreCompile(UBlueprint* Blueprint, const FKismetCompilerOptions& CompileOptions)
	{
		PreCompile(Blueprint);
	}

	virtual bool CanCompile(const UBlueprint* Blueprint) = 0;
	virtual void Compile(UBlueprint* Blueprint, const FKismetCompilerOptions& CompileOptions, FCompilerResultsLog& Results) = 0;
	
	virtual void PostCompile(UBlueprint* Blueprint, const FKismetCompilerOptions& CompileOptions)
	{
		PostCompile(Blueprint);
	}

	virtual bool GetBlueprintTypesForClass(UClass* ParentClass, UClass*& OutBlueprintClass, UClass*& OutBlueprintGeneratedClass) const
	{
		OutBlueprintClass = nullptr;
		OutBlueprintGeneratedClass = nullptr;
		return false;
	}

protected:
	virtual void PreCompile(UBlueprint* Blueprint) { }
	virtual void PostCompile(UBlueprint* Blueprint) { }
};

class IKismetCompilerInterface : public IModuleInterface
{
public:
	/**
	 * Synchronizes Blueprint's GeneratedClass's properties with the NewVariable declarations in the blueprint
	 * Used on load to ensure that all properties are present for instances.
	 * 
	 * @param Blueprint The blueprint that may be missing variables
	 */
	virtual void RefreshVariables(UBlueprint* Blueprint)=0;

	/**
	 * Compiles a user defined structure.
	 *
	 * @param	Struct		The structure to compile.
	 * @param	Results  	The results log for warnings and errors.
	 */
	virtual void CompileStructure(class UUserDefinedStruct* Struct, FCompilerResultsLog& Results)=0;

	/**
	 * Attempts to recover a corrupted blueprint package.
	 *
	 * @param	Blueprint	The blueprint to recover.
	 */
	virtual void RecoverCorruptedBlueprint(class UBlueprint* Blueprint)=0;

	/**
	 * Clears the blueprint's generated classes, and consigns them to oblivion
	 *
	 * @param	Blueprint	The blueprint to clear the classes for
	 */
	virtual void RemoveBlueprintGeneratedClasses(class UBlueprint* Blueprint)=0;

	/**
	 * Gets a list of all compilers for blueprints.  You can register new compilers through this list.
	 */
	virtual TArray<IBlueprintCompiler*>& GetCompilers() = 0;

	/** Facilities for establishing mappings between UClasses, UBlueprints, and UBlueprintGenerated Classes*/
	virtual void OverrideBPTypeForClass(UClass* Class, TSubclassOf<UBlueprint> BlueprintType) = 0;
	UE_DEPRECATED(5.4, "Conditional overrides in editor have been deprecated - make a new sentinel type if required to keep UBlueprint mappings unambiguous")
	virtual void OverrideBPTypeForClassInEditor(UClass* Class, TSubclassOf<UBlueprint> BlueprintType) = 0;
	virtual void OverrideBPGCTypeForBPType(TSubclassOf<UBlueprint> BlueprintType, TSubclassOf<UBlueprintGeneratedClass> BPGCType) = 0;

	virtual void ValidateBPAndClassType(UBlueprint* BP, FCompilerResultsLog& OutResults) = 0;

	/**
	 * Get the blueprint class and generated blueprint class for a particular class type.  Not every
	 * blueprint is a normal UBlueprint, like UUserWidget blueprints should be UWidgetBlueprints.
	 */
	virtual void GetBlueprintTypesForClass(UClass* ParentClass, UClass*& OutBlueprintClass, UClass*& OutBlueprintGeneratedClass) const = 0;
	virtual void GetSubclassesWithDifferingBlueprintTypes(UClass* Class, TSet<const UClass*>& OutMismatchedSubclasses) const = 0;
};


