// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FCompilerResultsLog;

class FUserDefinedStructureCompilerUtils
{
public:
	// ASSUMPTION, structure doesn't need to be renamed or removed
	static void CompileStruct(class UUserDefinedStruct* Struct, class FCompilerResultsLog& MessageLog, bool bForceRecompile);

	KISMETCOMPILER_API static void ReplaceStructWithTempDuplicateByPredicate(
		UUserDefinedStruct* StructureToReinstance,
		TFunctionRef<bool(FStructProperty* InStructProperty)> ShouldReplaceStructInStructProperty,
		TFunctionRef<void(UStruct* InStruct)> PostReplace);

	KISMETCOMPILER_API static void ReplaceStructWithTempDuplicateSafe(UUserDefinedStruct* Struct);
};

